{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Geometry Mask",
	"Description": "",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "GeometryMask",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "GeometryMaskEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
