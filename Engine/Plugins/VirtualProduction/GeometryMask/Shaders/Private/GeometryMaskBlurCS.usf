// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Public/Platform.ush"

#define TILE_SIZE 8
#define MAX_SAMPLES 127
#define HALF_MAX_SAMPLES MAX_SAMPLES / 2

int SampleCount;
float4 WeightsAndOffsets[HALF_MAX_SAMPLES];
float4 BufferSizeAndDirection;

uint2 InputDimensions;
Texture2D InputTexture;
SamplerState InputSampler;
RWTexture2D<float4> OutputTexture;

float4 GetSample(float Weight, float Offset, float2 UV)
{
	const float2 MinUV = float2(0, 0);
	const float2 MaxUV = float2(1.0, 1.0);
	const float2 Direction = BufferSizeAndDirection.zw;
	const float2 BufferSize = BufferSizeAndDirection.xy;
	UV *= BufferSize;

	const float2 UVOffset = float2(Offset * BufferSize.x * Direction.x, Offset * BufferSize.y * Direction.y);

	return (InputTexture.SampleLevel(InputSampler, clamp(UV + UVOffset, MinUV, MaxUV), 0) * Weight
		 + InputTexture.SampleLevel(InputSampler, clamp(UV - UVOffset, MinUV, MaxUV), 0) * Weight);
}

// Loosely based on Engine/Shaders/Private/SlatePostProcessPixelShader.usf
[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupId : SV_GroupID)
{
	uint2 PixelCoord = DispatchThreadId.xy;

	float X = (float)PixelCoord.x + 0.5;
	float Y = (float)PixelCoord.y + 0.5;
	float2 UV = float2(X, Y);

	const float2 MinUV = float2(0, 0);
	const float2 MaxUV = float2(1.0, 1.0);

	float4 Color = InputTexture.SampleLevel(InputSampler, clamp(UV, MinUV, MaxUV), 0);

	// First offset is in zw
	{
		const float Weight = WeightsAndOffsets[0].z;
		const float Offset = WeightsAndOffsets[0].w;

		Color += GetSample(Weight, Offset, UV);
	}

	for (int SampleIdx = 2; SampleIdx < SampleCount; SampleIdx += 2)
	{
		int ParamIdx = SampleIdx / 2;
		{
			const float Weight = WeightsAndOffsets[ParamIdx].x;
			const float Offset = WeightsAndOffsets[ParamIdx].y;

			Color += GetSample(Weight, Offset, UV);
		}

		{
			const float Weight = WeightsAndOffsets[ParamIdx].z;
			const float Offset = WeightsAndOffsets[ParamIdx].w;

			Color += GetSample(Weight, Offset, UV);
		}
	}

	Color.a = 1.0;

	OutputTexture[PixelCoord] = Color;
}
