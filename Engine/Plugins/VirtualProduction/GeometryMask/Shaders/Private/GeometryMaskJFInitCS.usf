// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

uint2 InputDimensions;
float2 OneOverInputDimensions;
float2 UVRatioAdjustment;
Texture2D InputTexture;
SamplerState InputSampler;
RWBuffer<float4> OutputBuffer;

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupId : SV_GroupID)
{
	const uint2 PixelCoord = DispatchThreadId.xy;
	if (PixelCoord.x > InputDimensions.x - 1 || PixelCoord.y > InputDimensions.y - 1)
	{
		return;
	}

	float X = (float)PixelCoord.x + 0.5;
	float Y = (float)PixelCoord.y + 0.5;
	const float2 UV = float2(X, Y) * OneOverInputDimensions * UVRatioAdjustment;

	const uint RowLength = InputDimensions.x;	
	const uint PixelCoord1D = (PixelCoord.y * RowLength) + (PixelCoord.x);

	static int SobelX[] =
	{
		1, 0, -1,
		2, 0, -2,
		1, 0, -1
	};

	static int SobelY[] =
	{
		1,  2,  1,
		0,  0,  0,
	   -1, -2, -1
   };

	static uint2 NeighborOffsetIndices[] =
	{
		{-1,  1},  {0,  1},  {1,  1},
		{-1,  0},  {0,  0},  {1,  0},
		{-1, -1},  {0, -1},  {1, -1}
	};
	
	float4 Input = InputTexture.Load(int3(PixelCoord, 0));
	float Edges = 0;
	int Mask = 0;

	OutputBuffer[PixelCoord1D] = float4(0,0,0,0);

	float GradientX = 0;
	float GradientY = 0;

	[unroll]
	for (int HorizontalIdx = 0; HorizontalIdx < 3; ++HorizontalIdx)
	{
		[unroll]
		for (int VerticalIdx = 0; VerticalIdx < 3; ++VerticalIdx)
		{
			uint NeighborIdx = (VerticalIdx * KERNEL_SIZE) + HorizontalIdx;
			uint2 NeighborPixelCoord = PixelCoord + NeighborOffsetIndices[NeighborIdx];
			NeighborPixelCoord = clamp(NeighborPixelCoord, uint2(0,0), uint2(InputDimensions.x, InputDimensions.y));

			// > 0 ensures treats the input as binary
			const float NeighborSample = InputTexture.Load(int3(NeighborPixelCoord, 0))[0] > 0;

			GradientX += SobelX[NeighborIdx] * NeighborSample;
			GradientY += SobelY[NeighborIdx] * NeighborSample;
		}
	}

	Edges = max(abs(GradientX), abs(GradientY));
	Mask = min(1, Edges);

	// @note: This is specific to JFA requirements
	OutputBuffer[PixelCoord1D] = float4(UV * Mask, Edges, Input[0]);
}
