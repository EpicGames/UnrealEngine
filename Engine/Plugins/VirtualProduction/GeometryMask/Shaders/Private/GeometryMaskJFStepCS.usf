// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

uint2 InputDimensions;
float2 OneOverInputDimensions;
float2 UVRatioAdjustment;
uint StepSize;
Buffer<float4> InputBuffer;
RWBuffer<float4> OutputBuffer;

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupId : SV_GroupID)
{
	uint2 PixelCoord = DispatchThreadId.xy;
	if (PixelCoord.x > InputDimensions.x - 1 || PixelCoord.y > InputDimensions.y - 1)
	{
		return;
	}

	const uint RowLength = InputDimensions.x;
	uint PixelCoord1D = (PixelCoord.y * RowLength) + (PixelCoord.x);

	static uint2 NeighborOffsetIndices[] =
	{
		{-1,  1},  {0,  1},  {1,  1},
		{-1,  0},  {0,  0},  {1,  0},
		{-1, -1},  {0, -1},  {1, -1}
	};

	float MinDistance = 100000000;
	float4 MinSample = float4(0,0,0,0);

	float X = (float)PixelCoord.x + 0.5;
	float Y = (float)PixelCoord.y + 0.5;
	const float2 UV = float2(X, Y) * OneOverInputDimensions * UVRatioAdjustment;

	float4 Input = InputBuffer[PixelCoord1D];
	const uint2 StepSize2 = uint2(StepSize, StepSize);

	[unroll]
	for (int HorizontalIdx = 0; HorizontalIdx < 3; ++HorizontalIdx)
	{
		[unroll]
		for (int VerticalIdx = 0; VerticalIdx < 3; ++VerticalIdx)
		{
			uint NeighborIdx = (VerticalIdx * KERNEL_SIZE) + HorizontalIdx;
			uint2 NeighborPixelCoord = PixelCoord + (NeighborOffsetIndices[NeighborIdx] * StepSize2);

			// Prevents border banding
			NeighborPixelCoord = min(InputDimensions - 1, max(uint2(1,1), NeighborPixelCoord));

			uint NeighborPixelCoord1D = (NeighborPixelCoord.y * RowLength) + (NeighborPixelCoord.x);

			float4 NeighborSample = InputBuffer[NeighborPixelCoord1D];
			if (NeighborSample.x + NeighborSample.y == 0.0)
			{
				continue;
			}

			const float Distance = length(NeighborSample.xy - UV);
			if (Distance < MinDistance)
			{
				MinDistance = Distance;
				MinSample = NeighborSample;
			}
		}
	}

	MinSample.a = Input.a;
	OutputBuffer[PixelCoord1D] = MinSample;
}
