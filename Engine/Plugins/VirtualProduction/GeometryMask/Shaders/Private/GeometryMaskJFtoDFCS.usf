// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"

#define DEBUG_SOBEL 0

uint2 InputDimensions;
float2 OneOverInputDimensions;
float2 UVRatioAdjustment;
float StepDistanceMultiplier; // To normalize
Texture2D OriginalInputTexture;
SamplerState OriginalInputSampler;
Buffer<float4> InputBuffer;
RWTexture2D<float4> OutputTexture;

[numthreads(TILE_SIZE, TILE_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID,
	uint3 GroupId : SV_GroupID)
{
	uint2 PixelCoord = DispatchThreadId.xy;
	if (PixelCoord.x > InputDimensions.x - 1 || PixelCoord.y > InputDimensions.y - 1)
	{
		return;
	}

	const uint RowLength = InputDimensions.x;	
	const uint PixelCoord1D = (PixelCoord.y * RowLength) + (PixelCoord.x);

	float X = (float)PixelCoord.x + 0.5;
	float Y = (float)PixelCoord.y + 0.5;
	const float2 UV = float2(X, Y) * OneOverInputDimensions * UVRatioAdjustment;

	float4 Outline = OriginalInputTexture.Load(int3(PixelCoord, 0));
	float4 Mask = float4(1,1,1,1);

#if DEBUG_SOBEL
	float4 Sample = InputBuffer[PixelCoord1D];
	Outline.x = saturate(distance(UV, Sample.xy));
#else
	float4 Sample = InputBuffer[PixelCoord1D];

	float MaskValue = (Sample.x + Sample.y + Sample.a > 0.001);

	float OutlineValue = saturate(distance(UV, Sample.xy) * StepDistanceMultiplier);

	// Set range such that inner goes from 0.0-0.5, outer goes from 0.5-1.0
	OutlineValue = mad(OutlineValue, 0.5, 0.5);
	OutlineValue = 1.0 - abs(OutlineValue - saturate(Sample.a));
	Mask.x = MaskValue; Outline.x = OutlineValue;
#endif

	Outline = lerp(0, Outline, Mask);

	OutputTexture[PixelCoord] = Outline;
}
