// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class GeometryMask : ModuleRules
{
	public GeometryMask(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DeveloperSettings",
				"GeometryCore",
				"GeometryFramework",
				"Projects",
				"RHI",
				"RenderCore",
				"Renderer",
			});

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"LevelEditor",
					"UnrealEd",
				});
		}
	}
}
