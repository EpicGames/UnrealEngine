// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskCanvas.h"
#include "Algo/RemoveIf.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GeometryMaskCanvasResource.h"
#include "GeometryMaskCanvasSharedData.h"
#include "GeometryMaskModule.h"
#include "GeometryMaskWriter.h"
#include "IGeometryMaskWriteInterface.h"

const FName UGeometryMaskCanvas::ApplyBlurPropertyName = GET_MEMBER_NAME_CHECKED(UGeometryMaskCanvas, bApplyBlur);
const FName UGeometryMaskCanvas::BlurStrengthPropertyName = GET_MEMBER_NAME_CHECKED(UGeometryMaskCanvas, BlurStrength);
const FName UGeometryMaskCanvas::ApplyFeatherPropertyName = GET_MEMBER_NAME_CHECKED(UGeometryMaskCanvas, bApplyFeather);
const FName UGeometryMaskCanvas::OuterFeatherRadiusPropertyName = GET_MEMBER_NAME_CHECKED(UGeometryMaskCanvas, InnerFeatherRadius);
const FName UGeometryMaskCanvas::InnerFeatherRadiusPropertyName = GET_MEMBER_NAME_CHECKED(UGeometryMaskCanvas, OuterFeatherRadius);

#if WITH_EDITOR
void UGeometryMaskCanvas::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	UObject::PostEditChangeProperty(InPropertyChangedEvent);

	static TSet<FName> UpdateRenderParameterProperties =
	{
		ApplyBlurPropertyName,
		BlurStrengthPropertyName,
		ApplyFeatherPropertyName,
		OuterFeatherRadiusPropertyName,
		InnerFeatherRadiusPropertyName
	};

	FName PropertyName = InPropertyChangedEvent.GetPropertyName();
	if (UpdateRenderParameterProperties.Contains(PropertyName))
	{
		UpdateRenderParameters();
	}
}
#endif

ULevel* UGeometryMaskCanvas::GetLevel() const
{
	return CanvasId.Level.ResolveObjectPtr();
}

const TArray<TWeakInterfacePtr<IGeometryMaskWriteInterface>>& UGeometryMaskCanvas::GetWriters() const
{
	return Writers;
}

void UGeometryMaskCanvas::AddWriter(const TScriptInterface<IGeometryMaskWriteInterface>& InWriter)
{
	if (!ensure(InWriter))
	{
		return;
	}

	RemoveInvalidWriters();

	// Writers is empty, but about to become activated
	if (Writers.IsEmpty())
	{
		OnActivated().ExecuteIfBound();
	}

	if (Writers.ContainsByPredicate([InWriter](const TWeakInterfacePtr<IGeometryMaskWriteInterface>& InExistingWriter)
	{
		return InExistingWriter.IsValid() && InExistingWriter.GetObject() == InWriter.GetObject();
	}))
	{
		return;
	}

	Writers.Add(InWriter.GetObject());
}

void UGeometryMaskCanvas::AddWriters(const TArray<TScriptInterface<IGeometryMaskWriteInterface>>& InWriters)
{
	if (InWriters.IsEmpty())
	{
		return;
	}

	RemoveInvalidWriters();
	for (const TScriptInterface<IGeometryMaskWriteInterface>& Writer : InWriters)
	{
		AddWriter(Writer);
	}
}

void UGeometryMaskCanvas::RemoveWriter(const TScriptInterface<IGeometryMaskWriteInterface>& InWriter)
{
	if (!ensure(InWriter))
	{
		return;
	}

	RemoveInvalidWriters();

	Writers.SetNum(Algo::RemoveIf(Writers, [InWriter](const TWeakInterfacePtr<IGeometryMaskWriteInterface>& InExistingWriter)
	{
		return InExistingWriter.GetObject() == InWriter.GetObject();
	}));
}

int32 UGeometryMaskCanvas::GetNumWriters() const
{
	return Writers.Num();
}

bool UGeometryMaskCanvas::IsDefaultCanvas() const
{
	return CanvasId.IsDefault();
}

void UGeometryMaskCanvas::Free()
{
	Writers.Empty();
	OnDeactivated().ExecuteIfBound();
}

UCanvasRenderTarget2D* UGeometryMaskCanvas::GetTexture() const
{
	return nullptr;
}

UTextureRenderTarget2DArray* UGeometryMaskCanvas::GetRenderTarget() const
{
	return CanvasResource ? CanvasResource->GetRenderTarget() : nullptr;
}

int16 UGeometryMaskCanvas::GetRenderTargetSliceIndex() const
{
	return CanvasResource ? CanvasResource->GetRenderTargetSliceIndex() : -1;
}

int32 UGeometryMaskCanvas::BP_GetRenderTargetSliceIndex() const
{
	return GetRenderTargetSliceIndex();
}

const FGeometryMaskCanvasId& UGeometryMaskCanvas::GetCanvasId() const
{
	return CanvasId;
}

FName UGeometryMaskCanvas::GetCanvasName() const
{
	return CanvasName;
}

bool UGeometryMaskCanvas::IsBlurApplied() const
{
	return bApplyBlur;
}

void UGeometryMaskCanvas::SetApplyBlur(const bool bInValue)
{
	if (bInValue != bApplyBlur)
	{
		bApplyBlur = bInValue;
		UpdateRenderParameters();
	}
}

double UGeometryMaskCanvas::GetBlurStrength() const
{
	return BlurStrength;
}

void UGeometryMaskCanvas::SetBlurStrength(const double InValue)
{
	if (!FMath::IsNearlyEqual(InValue, BlurStrength))
	{
		BlurStrength = InValue;
		UpdateRenderParameters();
	}
}

bool UGeometryMaskCanvas::IsFeatherApplied() const
{
	return bApplyFeather;
}

void UGeometryMaskCanvas::SetApplyFeather(const bool bInValue)
{
	if (bInValue != bApplyFeather)
	{
		bApplyFeather = bInValue;
		UpdateRenderParameters();
	}
}

int32 UGeometryMaskCanvas::GetOuterFeatherRadius() const
{
	return OuterFeatherRadius;
}

void UGeometryMaskCanvas::SetOuterFeatherRadius(const int32 InValue)
{
	if (InValue != OuterFeatherRadius)
	{
		OuterFeatherRadius = InValue;
		UpdateRenderParameters();
	}
}

int32 UGeometryMaskCanvas::GetInnerFeatherRadius() const
{
	return InnerFeatherRadius;
}

void UGeometryMaskCanvas::SetInnerFeatherRadius(const int32 InValue)
{
	if (InValue != InnerFeatherRadius)
	{
		InnerFeatherRadius = InValue;
		UpdateRenderParameters();
	}
}

EGeometryMaskColorChannel UGeometryMaskCanvas::GetColorChannel() const
{
	return EGeometryMaskColorChannel::Red;
}

void UGeometryMaskCanvas::RemoveInvalidWriters()
{
	if (Writers.IsEmpty())
	{
		return;
	}

	Writers.SetNum(Algo::RemoveIf(Writers, [](const TWeakInterfacePtr<IGeometryMaskWriteInterface>& InWriter)
	{
		return !InWriter.IsValid() || !IsValid(InWriter.GetObject());
	}));

	if (Writers.IsEmpty())
	{
		// No writers after cleanup, flag as deactivated
		OnDeactivated().ExecuteIfBound();
	}
}

void UGeometryMaskCanvas::OnDrawToCanvas(const FGeometryMaskDrawingContext& InDrawingContext, FSceneView& InView, FCanvas* InCanvas)
{
	if (!ensure(InCanvas))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskCanvas::OnDrawToCanvas);

	// Sort so subtract happens after additive, etc.
	SortWriters();

	for (const TWeakInterfacePtr<IGeometryMaskWriteInterface>& WriterWeak : Writers)
	{
		IGeometryMaskWriteInterface* const Writer = WriterWeak.Get();
		if (!Writer || !Writer->IsMaskWriterEnabled())
		{
			continue;
		}

		// Draw the mask primitives to canvas
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(IGeometryMaskWriteInterface::OnDrawToCanvas);
			Writer->DrawToCanvas(InCanvas);
		}

		// Add the mask primitives to the hidden list
		if (UE::GeometryMask::FMaskWriter* MaskWriter = Writer->GetMaskWriter())
		{
			MaskWriter->ForEachMaskPrimitive(
				[&InView](TNotNull<const UPrimitiveComponent*> InComponent)
				{
					InView.HiddenPrimitives.Add(InComponent->GetPrimitiveSceneId());
				});
		}
	}
}

void UGeometryMaskCanvas::UpdateRenderParameters()
{
	if (CanvasResource)
	{
		CanvasResource->UpdateRenderParameters(bApplyBlur, BlurStrength, bApplyFeather, OuterFeatherRadius, InnerFeatherRadius);
	}
}

void UGeometryMaskCanvas::Initialize(const ULevel* InLevel, FName InCanvasName)
{
}

void UGeometryMaskCanvas::Initialize(const FInitParams& InInitParams)
{
	CanvasName = InInitParams.CanvasName;
	CanvasId = FGeometryMaskCanvasId(InInitParams.Level, CanvasName);

	CanvasResource = NewObject<UGeometryMaskCanvasResource>(this);
	CanvasResource->Initialize(InInitParams.RenderTarget, InInitParams.SliceIndex, InInitParams.SharedData);
	UpdateRenderParameters();
}

void UGeometryMaskCanvas::Update(const ULevel* InLevel,FSceneView& InView)
{
	RemoveInvalidWriters();
}

TSharedPtr<FGeometryMaskCanvasSharedData> UGeometryMaskCanvas::GetSharedData() const
{
	if (CanvasResource)
	{
		return CanvasResource->GetSharedData();
	}
	return nullptr;
}

void UGeometryMaskCanvas::AssignResource(UGeometryMaskCanvasResource* InResource, const EGeometryMaskColorChannel InColorChannel)
{
}

void UGeometryMaskCanvas::FreeResource()
{
}

FName UGeometryMaskCanvas::GetApplyBlurPropertyName()
{
	return ApplyBlurPropertyName;
}

FName UGeometryMaskCanvas::GetBlurStrengthPropertyName()
{
	return BlurStrengthPropertyName;
}

FName UGeometryMaskCanvas::GetApplyFeatherPropertyName()
{
	return ApplyFeatherPropertyName;
}

FName UGeometryMaskCanvas::GetOuterFeatherRadiusPropertyName()
{
	return OuterFeatherRadiusPropertyName;
}

FName UGeometryMaskCanvas::GetInnerFeatherRadiusPropertyName()
{
	return InnerFeatherRadiusPropertyName;
}

void UGeometryMaskCanvas::SortWriters()
{
	Algo::SortBy(Writers, [](const TWeakInterfacePtr<IGeometryMaskWriteInterface>& InWriter)
	{
		if (const IGeometryMaskWriteInterface* Writer = InWriter.Get())
		{
			return Writer->GetParameters().OperationType;
		}

		return EGeometryMaskCompositeOperation::Num;
	});
}
