// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskCanvasActor.h"

#include "Engine/Engine.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "GeometryMaskWriteComponent.h"

AGeometryMaskCanvasActor::AGeometryMaskCanvasActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
	SetActorTickEnabled(true);

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif

	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("Root"));

	Canvas = nullptr;
}

UCanvasRenderTarget2D* AGeometryMaskCanvasActor::GetTexture() const
{
	return nullptr;
}

UTextureRenderTarget2DArray* AGeometryMaskCanvasActor::GetRenderTarget() const
{
	return Canvas ? Canvas->GetRenderTarget() : nullptr;
}

void AGeometryMaskCanvasActor::BeginPlay()
{
	Super::BeginPlay();

	TryResolveCanvas();
}

void AGeometryMaskCanvasActor::PostLoad()
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		TryResolveCanvas();
	}
}

#if WITH_EDITOR
void AGeometryMaskCanvasActor::RerunConstructionScripts()
{
	Super::RerunConstructionScripts();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		TryResolveCanvas();
	}
}
#endif

bool AGeometryMaskCanvasActor::TryResolveCanvas()
{
	if (!Canvas)
	{
		if (UGeometryMaskWorldSubsystem* Subsystem = GetWorld()->GetSubsystem<UGeometryMaskWorldSubsystem>())
		{
			Canvas = Subsystem->GetNamedCanvas(GetLevel(), CanvasName, this);
		}
	}
	
	FindWriters();
	
	return Canvas != nullptr;
}

void AGeometryMaskCanvasActor::FindWriters()
{
	// Find actors that implement, or contain components that implement IGeometryMaskWriteInterface
	TArray<AActor*> ChildActors;
	GetAttachedActors(ChildActors, true, true);

	TArray<TScriptInterface<IGeometryMaskWriteInterface>> FoundWriters;
	FoundWriters.Reserve(ChildActors.Num());

	for (const TObjectPtr<AActor> ChildActor : ChildActors)
	{
		if (ChildActor->Implements<UGeometryMaskWriteInterface>())
		{
			FoundWriters.Add(ChildActor);
		}

		FoundWriters.Append(ChildActor->GetComponentsByInterface(UGeometryMaskWriteInterface::StaticClass()));
	}

	Writers.Reset();
	Writers = FoundWriters;

	if (Canvas)
	{
		Canvas->AddWriters(Writers);
	}
}

bool AGeometryMaskCanvasActor::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	return InFunc(CanvasName);
}
