// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskCanvasResource.h"

#include "Algo/MaxElement.h"
#include "Engine/Canvas.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Level.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/World.h"
#include "GeometryMaskCanvasSharedData.h"
#include "GeometryMaskCanvasUtils.h"
#include "GeometryMaskModule.h"
#include "GeometryMaskSettings.h"
#include "GeometryMaskTypes.h"
#include "GeometryMaskWorldSubsystem.h"
#include "SceneView.h"
#include "Shaders/GeometryMaskPostProcess.h"
#include "Shaders/GeometryMaskPostProcess_Blur.h"
#include "Shaders/GeometryMaskPostProcess_DistanceField.h"
#include "TextureResource.h"
#include "UObject/Package.h"

namespace UE::GeometryMask::Private
{
	void OverscanProjectionMatrix(FMatrix& InOutMatrix, const FIntPoint& InSize, const int32 InPadding)
	{
		const FVector2f Multiplier(
			static_cast<float>(InSize.X) / static_cast<float>(InSize.X + InPadding),
			static_cast<float>(InSize.Y) / static_cast<float>(InSize.Y + InPadding));

		// Original Calc - we simply scale each
		// [0][0] = MultFOVX / FMath::Tan(HalfFOVX)
		// [1][1] = MultFOVY / FMath::Tan(HalfFOVY)
		
		InOutMatrix.M[0][0] *= Multiplier.X;
		InOutMatrix.M[1][1] *= Multiplier.Y;
	}
}

UGeometryMaskCanvasResource::UGeometryMaskCanvasResource()
	: DefaultDrawingContext()
{
	FGeometryMaskPostProcessParameters_Blur PostProcessParameters_Blur;
	PostProcess_Blur = MakeShared<FGeometryMaskPostProcess_Blur>(PostProcessParameters_Blur);

	FGeometryMaskPostProcessParameters_DistanceField PostProcessParameters_DistanceField;
	PostProcess_DistanceField = MakeShared<FGeometryMaskPostProcess_DistanceField>(PostProcessParameters_DistanceField);
}

void UGeometryMaskCanvasResource::BeginDestroy()
{
	Canvas.Reset();
	Super::BeginDestroy();
}

void UGeometryMaskCanvasResource::Initialize(UTextureRenderTarget2DArray* InRenderTarget, int32 InSliceIndex, const TSharedPtr<FGeometryMaskCanvasSharedData>& InSharedData)
{
	RenderTarget = InRenderTarget;
	RenderTargetSliceIndex = InSliceIndex;
	SharedData = InSharedData;
}

const EGeometryMaskColorChannel UGeometryMaskCanvasResource::GetNextAvailableColorChannel() const
{
	return EGeometryMaskColorChannel::None;
}

bool UGeometryMaskCanvasResource::Checkout(const EGeometryMaskColorChannel InColorChannel,const FGeometryMaskCanvasId& InRequestingCanvasId)
{
	if (!ensure(InColorChannel != EGeometryMaskColorChannel::Num && InColorChannel != EGeometryMaskColorChannel::None))
	{
		// Input channel invalid
		return false;
	}

	RemoveInvalidDrawingContexts();

	FGeometryMaskDrawingContext* DrawingContext = GetDrawingContextForCanvas(InRequestingCanvasId);
	if (!ensure(DrawingContext))
	{
		// World is probably invalid
		return false;
	}

	return true;
}

bool UGeometryMaskCanvasResource::Checkin(const FGeometryMaskCanvasId& InRequestingCanvasId)
{
	// The requesting canvas name wasn't present in this Resource
	RemoveInvalidDrawingContexts();
	return false;
}

int32 UGeometryMaskCanvasResource::Compact()
{
	return 0;
}

int32 UGeometryMaskCanvasResource::GetNumChannelsUsed() const
{
	return 1;
}

bool UGeometryMaskCanvasResource::IsAnyChannelUsed() const
{
	return true;
}

TArray<FGeometryMaskCanvasId> UGeometryMaskCanvasResource::GetDependentCanvasIds() const
{
	return {};
}

const TSharedPtr<FGeometryMaskCanvasSharedData>& UGeometryMaskCanvasResource::GetSharedData() const
{
	return SharedData;
}

bool UGeometryMaskCanvasResource::UpdateViewportSize()
{
	if (!SharedData.IsValid())
	{
		return false;
	}

	SharedData->ViewportPadding = FMath::Max(SharedData->ViewportPadding, GetRequiredViewportPadding());

	// Update based on max required of all drawing contexts
	for (FGeometryMaskDrawingContext& DrawingContext : DrawingContextCache)
	{
		SharedData->ViewportSize = SharedData->ViewportSize.ComponentMax(DrawingContext.ViewportSize);
	}

	if (SharedData->ViewportSize.Size() == 0)
	{
		return false;
	}

	MaxViewportSize = SharedData->ViewportSize;

	const FIntPoint PaddedViewportSize = MaxViewportSize + SharedData->ViewportPadding;
	int32 SizeX = PaddedViewportSize.X;
	int32 SizeY = PaddedViewportSize.Y;

	if (const float RatioX = static_cast<float>(SizeX) / UE::GeometryMask::MaxTextureSize;
		RatioX > 1.0)
	{
		// Width too big, cap to max and reduce height proportionally
		SizeX = UE::GeometryMask::MaxTextureSize;
		SizeY /= RatioX;
	}

	if (const float RatioY = static_cast<float>(SizeY) / UE::GeometryMask::MaxTextureSize;
		RatioY > 1.0)
	{
		// Height too big, cap to max and reduce width proportionally
		SizeY = UE::GeometryMask::MaxTextureSize;
		SizeX /= RatioY;
	}

	SharedData->TextureSize = FIntPoint(SizeX, SizeY);

	if (!RenderTarget || (RenderTarget->SizeX == SizeX && RenderTarget->SizeY == SizeY))
	{
		return false;
	}

	// Update RT size to viewport size
	RenderTarget->SizeX = SizeX;
	RenderTarget->SizeY = SizeY;
	UE::GeometryMask::UpdateRenderTarget(RenderTarget);
	return true;
}

void UGeometryMaskCanvasResource::UpdateRenderParameters(const EGeometryMaskColorChannel InColorChannel,const bool bInApplyBlur,const double InBlurStrength,bool bInApplyFeather,int32 InOuterFeatherRadius,int32 InInnerFeatherRadius)
{
	UpdateRenderParameters(bInApplyBlur, InBlurStrength, bInApplyFeather, InOuterFeatherRadius, InInnerFeatherRadius);
}

void UGeometryMaskCanvasResource::UpdateRenderParameters(bool bInApplyBlur, double InBlurStrength, bool bInApplyFeather, int32 InOuterFeatherRadius, int32 InInnerFeatherRadius)
{
	if (RenderTargetSliceIndex < 0)
	{
		UE_LOGF(LogGeometryMask, Error, "RenderTargetSliceIndex wasn't valid for setting shader parameters (was '%d', expected a valid index 0 or above", RenderTargetSliceIndex);
		return;
	}

	FGeometryMaskPostProcessParameters_Blur BlurParameters = PostProcess_Blur->GetParameters();
	{
		BlurParameters.SliceIndex = RenderTargetSliceIndex;
		BlurParameters.bApplyBlur = bInApplyBlur && InBlurStrength > 0.0;
		BlurParameters.BlurStrength = InBlurStrength;

		PostProcess_Blur->SetParameters(BlurParameters);
	}

	FGeometryMaskPostProcessParameters_DistanceField DFParameters = PostProcess_DistanceField->GetParameters();
	{
		DFParameters.SliceIndex = RenderTargetSliceIndex;
		DFParameters.bCalculateDF = bInApplyFeather && (InOuterFeatherRadius + InInnerFeatherRadius) > 0;
		DFParameters.Radius = FMath::Max(InOuterFeatherRadius, InInnerFeatherRadius);

		PostProcess_DistanceField->SetParameters(DFParameters);
	}

	bApplyBlur = BlurParameters.bApplyBlur;
	bApplyDF = DFParameters.bCalculateDF;

	if (SharedData.IsValid())
	{
		SharedData->Reset();
	}

	// Effects may have changed viewport padding
    UpdateViewportSize();
}

void UGeometryMaskCanvasResource::ResetRenderParameters(EGeometryMaskColorChannel InColorChannel)
{
	ResetRenderParameters();
}

void UGeometryMaskCanvasResource::ResetRenderParameters()
{
	UpdateRenderParameters(/*bApplyBlur*/false, /*BlurStrength*/0.0, /*bApplyFeather*/false, /*OuterFeatherRadius*/0, /*InnerFeatherRadius*/0);
}

void UGeometryMaskCanvasResource::SetViewportSize(FGeometryMaskDrawingContext& InDrawingContext, const FIntPoint& InViewportSize)
{
	if (InViewportSize.Size() > 0 && InDrawingContext.ViewportSize != InViewportSize)
	{
		InDrawingContext.ViewportSize = InViewportSize;
		UpdateViewportSize();
	}
}

const FIntPoint& UGeometryMaskCanvasResource::GetMaxViewportSize() const
{
	return MaxViewportSize;
}

int32 UGeometryMaskCanvasResource::GetViewportPadding(const FGeometryMaskDrawingContext& InDrawingContext) const
{
	return GetRequiredViewportPadding();
}

int32 UGeometryMaskCanvasResource::GetSharedViewportPadding() const
{
	return SharedData.IsValid() ? SharedData->ViewportPadding : 0;
}

int32 UGeometryMaskCanvasResource::GetRequiredViewportPadding() const
{
	int32 MaxFeatherRadius = 0;
	{
		const FGeometryMaskPostProcessParameters_DistanceField& DistanceFieldParameters = PostProcess_DistanceField->GetParameters();
		if (DistanceFieldParameters.bCalculateDF)
		{
			MaxFeatherRadius = FMath::Max(MaxFeatherRadius, DistanceFieldParameters.Radius);
		}
	}

	int32 MaxBlurRadius = 0;
	{
		const FGeometryMaskPostProcessParameters_Blur& BlurParameters = PostProcess_Blur->GetParameters();
		if (BlurParameters.bApplyBlur)
		{
			MaxBlurRadius = FMath::Max(MaxBlurRadius, UE::GeometryMask::Internal::ComputeEffectiveKernelSize(BlurParameters.BlurStrength));
		}
	}

	return FMath::Max(MaxFeatherRadius, MaxBlurRadius);
}

UCanvasRenderTarget2D* UGeometryMaskCanvasResource::GetRenderTargetTexture()
{
	return nullptr;
}

UTextureRenderTarget2DArray* UGeometryMaskCanvasResource::GetRenderTarget() const
{
	return RenderTarget;
}

int16 UGeometryMaskCanvasResource::GetRenderTargetSliceIndex() const
{
	return RenderTargetSliceIndex;
}

FOnGeometryMaskCanvasDraw& UGeometryMaskCanvasResource::OnDrawToCanvas()
{
	static FOnGeometryMaskCanvasDraw OnDrawToCanvasDelegate;
	return OnDrawToCanvasDelegate;
}

void UGeometryMaskCanvasResource::Update(const ULevel* InLevel, FSceneView& InView, int32 InViewIndex)
{
	Draw(InLevel, InView, InViewIndex);
}

ETextureRenderTargetSampleCount UGeometryMaskCanvasResource::GetSampleCount() const
{
	// MSAA not supported for these effects (they need UAV access)
	return ETextureRenderTargetSampleCount::RTSC_1;
}

void UGeometryMaskCanvasResource::Draw(const ULevel* InLevel, FSceneView& InView, int32 InViewIndex)
{
	if (!InLevel || !InLevel->OwningWorld || !RenderTarget || RenderTargetSliceIndex == -1 || !RenderTarget->GameThread_GetRenderTargetResource())
	{
		return;
	}

	UGeometryMaskCanvas* const Owner = Cast<UGeometryMaskCanvas>(GetOuter());
	if (!Owner)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskCanvasResource::Draw);

	UE_LOGF(LogGeometryMask, VeryVerbose, "UGeometryMaskCanvasResource::Update Level: %ls", *InLevel->GetName());

	// First time initialization
	if (!Canvas.IsValid())
	{
		Canvas = MakeShared<FCanvas>(
			RenderTarget->GameThread_GetRenderTargetResource(),
			nullptr,
			InLevel->OwningWorld,
			InLevel->OwningWorld->GetFeatureLevel(),
			// Draw immediately so that interleaved SetVectorParameter (etc) function calls work as expected
			FCanvas::CDM_ImmediateDrawing);
	}

	// Update render target as it could've changed (e.g. when changing slice count)
	Canvas->SetSliceIndex(RenderTargetSliceIndex);
	Canvas->SetRenderTarget_GameThread(RenderTarget->GameThread_GetRenderTargetResource());

	FGeometryMaskDrawingContext* const DrawingContext = GetDrawingContextForLevel(InLevel, InViewIndex);
    check(DrawingContext); // Should always be valid - world was already checked

	// Begin
	{
		const FIntPoint ViewportSize = InView.UnconstrainedViewRect.Size();
		if (ViewportSize.SizeSquared() > 0 && DrawingContext->ViewportSize != ViewportSize)
		{
			DrawingContext->ViewportSize = ViewportSize;
		}

		// Update viewport size
		if (UpdateViewportSize())
		{
			Canvas->SetRenderTarget_GameThread(RenderTarget->GameThread_GetRenderTargetResource());
		}

		FMatrix ProjectionMatrix = InView.ViewMatrices.GetViewToClip();
		UE::GeometryMask::Private::OverscanProjectionMatrix(ProjectionMatrix, MaxViewportSize, /*Padding*/0);

		DrawingContext->ViewProjectionMatrix = InView.ViewMatrices.GetWorldToView() * ProjectionMatrix;
		DrawingContext->ViewProjectionMatrix.M[2][2] = UE_KINDA_SMALL_NUMBER; // Prevents div by zero later

		InLevel->OwningWorld->FlushDeferredParameterCollectionInstanceUpdates();

		Canvas->SetParentCanvasSize(FIntPoint(RenderTarget->SizeX, RenderTarget->SizeY));
		Canvas->Clear(RenderTarget->ClearColor);
		RenderTarget->UpdateResourceImmediate(/*Clear*/false);
	}

	// Contents
	{
		// Store current transform
		const FMatrix CanvasMatrix = Canvas->GetBottomTransform();

		// Set to World->Viewport transform
		Canvas->SetBaseTransform(DrawingContext->ViewProjectionMatrix);

		Owner->OnDrawToCanvas(*DrawingContext, InView, Canvas.Get());

		// Restore original transform
		Canvas->SetBaseTransform(CanvasMatrix);
	}

	// End
	{
		// Commit batch elements to the rendering thread
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskCanvasResource::Draw::Flush);
			Canvas->Flush_GameThread();
		}

		// Post Process
		if (bApplyDF || bApplyBlur)
		{
			FRenderTarget* CanvasRenderTarget = Canvas->GetRenderTarget();
			if (bApplyDF)
			{
				PostProcess_DistanceField->Execute(CanvasRenderTarget);
			}

			if (bApplyBlur)
			{
				PostProcess_Blur->Execute(CanvasRenderTarget);
			}
		}
	}
}

FGeometryMaskDrawingContext* UGeometryMaskCanvasResource::GetDrawingContextForLevel(const ULevel* InLevel, uint8 InSceneViewIndex)
{
	if (!ensure(InLevel))
	{
		return nullptr;
	}

	return &DrawingContextCache.FindOrAdd(FGeometryMaskDrawingContext(InLevel, InSceneViewIndex));
}

FGeometryMaskDrawingContext* UGeometryMaskCanvasResource::GetDrawingContextForCanvas(const FGeometryMaskCanvasId& InCanvasId)
{
	if (InCanvasId.IsDefault())
	{
		return &DefaultDrawingContext;
	}
	
	return GetDrawingContextForLevel(InCanvasId.Level.ResolveObjectPtr(), 0);
}

FGeometryMaskDrawingContext* UGeometryMaskCanvasResource::GetDrawingContextForChannel(EGeometryMaskColorChannel InColorChannel)
{
	return nullptr;
}

bool UGeometryMaskCanvasResource::RemoveInvalidDrawingContexts()
{
	const int32 NumCurrentItems = DrawingContextCache.Num();
	
	TSet<FGeometryMaskDrawingContext> ValidDrawingContexts;
	ValidDrawingContexts.Reserve(NumCurrentItems);
	
	for (FGeometryMaskDrawingContext& DrawingContext : DrawingContextCache)
	{
		if (DrawingContext.IsValid())
		{
			ValidDrawingContexts.Emplace(MoveTemp(DrawingContext));
		}
	}

	DrawingContextCache = ValidDrawingContexts;

	return NumCurrentItems != DrawingContextCache.Num();
}
