// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskCanvasUtils.h"
#include "RenderingThread.h"
#include "Engine/TextureRenderTarget2DArray.h"

namespace UE::GeometryMask
{

void UpdateRenderTarget(TNotNull<UTextureRenderTarget2DArray*> InRenderTarget)
{
	InRenderTarget->UpdateResource();

	// Flush RHI thread after creating texture render target to make sure that RHIUpdateTextureReference is executed before doing any rendering with it
	// This makes sure that Value->TextureReference.TextureReferenceRHI->GetReferencedTexture() is valid so that FUniformExpressionSet::FillUniformBuffer properly uses the texture for rendering, instead of using a fallback texture
	ENQUEUE_RENDER_COMMAND(FlushRHIThreadToUpdateTextureRenderTargetReference)(
		[](FRHICommandListImmediate& RHICmdList)
		{
			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
		});
}

} // UE::GeometryMask
