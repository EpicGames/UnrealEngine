// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"

class UTextureRenderTarget2DArray;

namespace UE::GeometryMask
{
	/** Updates the resource for the render target */
	void UpdateRenderTarget(TNotNull<UTextureRenderTarget2DArray*> InRenderTarget);

} // UE::GeometryMask
