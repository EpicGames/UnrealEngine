// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskModule.h"

#include "GeometryMaskSettings.h"
#include "HAL/IConsoleManager.h"
#include "IGeometryMaskModule.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ShaderCore.h"

DEFINE_LOG_CATEGORY(LogGeometryMask);

#define LOCTEXT_NAMESPACE "FGeometryMaskModule"

IGeometryMaskModule& IGeometryMaskModule::Get()
{
	return FModuleManager::LoadModuleChecked<IGeometryMaskModule>(UE_MODULE_NAME);
}

void FGeometryMaskModule::StartupModule()
{
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME)->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/GeometryMask"), PluginShaderDir);

	if (GIsEditor && !IsRunningCommandlet())
	{
		RegisterConsoleCommands();
	}
}

void FGeometryMaskModule::ShutdownModule()
{
	// Cleanup commands
	UnregisterConsoleCommands();
}

void FGeometryMaskModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("GeometryMask.DebugDF"),
		TEXT("0: Off, 1: Sobel, 2...StepN: Step0...StepN"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FGeometryMaskModule::ExecuteDebugDF),
		ECVF_Default
	));
}

void FGeometryMaskModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FGeometryMaskModule::ExecuteDebugDF(const TArray<FString>& InArgs)
{
#if WITH_EDITORONLY_DATA
	int32 Value = 0;
	
	UGeometryMaskSettings* Settings = GetMutableDefault<UGeometryMaskSettings>();
	check(Settings);
	
	if (!InArgs.IsEmpty())
	{
		if (InArgs[0].IsNumeric())
		{
			Value = FCString::Atoi(*InArgs[0]);
		}
	}

	Settings->DebugDF = Value;
#endif
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeometryMaskModule, GeometryMask)
