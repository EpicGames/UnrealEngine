// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGeometryMaskModule.h"

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogGeometryMask, Log, All);
DECLARE_STATS_GROUP(TEXT("GeometryMask"), STATGROUP_GeometryMask, STATCAT_Advanced);

class FGeometryMaskModule
	: public IGeometryMaskModule
{
public:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

private:
	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();
	
	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();

	void ExecuteDebugDF(const TArray<FString>& InArgs);

private:
	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;
};
