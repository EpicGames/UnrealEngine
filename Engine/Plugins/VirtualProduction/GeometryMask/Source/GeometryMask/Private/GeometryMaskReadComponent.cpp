// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskReadComponent.h"

#include "Engine/CanvasRenderTarget2D.h"
#include "GeometryMaskCanvas.h"

void UGeometryMaskReadComponent::SetParameters(FGeometryMaskReadParameters& InParameters)
{
	Parameters = InParameters;
	TryResolveCanvas();
}

bool UGeometryMaskReadComponent::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	return InFunc(Parameters.CanvasName);
}

bool UGeometryMaskReadComponent::TryResolveCanvas()
{
	return TryResolveNamedCanvas(Parameters.CanvasName);
}
