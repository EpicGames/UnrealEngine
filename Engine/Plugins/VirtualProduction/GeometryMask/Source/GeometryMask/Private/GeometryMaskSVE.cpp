// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskSVE.h"

#include "GeometryMaskModule.h"
#include "GeometryMaskSubsystem.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

FGeometryMaskSceneViewExtension::FGeometryMaskSceneViewExtension(
	const FAutoRegister& AutoRegister,
	UWorld* InWorld)
	: FWorldSceneViewExtension(AutoRegister, InWorld)
{
	UE_LOGF(LogGeometryMask, VeryVerbose, "SVE registered for world: %ls", *InWorld->GetName());
	
	GeometryMaskSubsystemWeak = GEngine->GetEngineSubsystem<UGeometryMaskSubsystem>();
}

void FGeometryMaskSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	if (UGeometryMaskSubsystem* Subsystem = GeometryMaskSubsystemWeak.Get())
	{
		Subsystem->Update(GetWorld(), InViewFamily);
	}
}

bool FGeometryMaskSceneViewExtension::IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const
{
	bool bIsActive = FWorldSceneViewExtension::IsActiveThisFrame_Internal(Context);
	if (!bIsActive)
	{
		return false;
	}

#if WITH_EDITOR
	if (GEditor)
	{
		if (const UWorld* const ContextWorld = Context.GetWorld())
		{
			if (GEditor->IsSimulatingInEditor() && ContextWorld->WorldType == EWorldType::Editor)
			{
				return true;
			}

			if (GEditor->PlayWorld)
			{
				bIsActive = ContextWorld->WorldType == EWorldType::PIE || ContextWorld->WorldType == EWorldType::Game;
			}
			else
			{
				bIsActive = ContextWorld->WorldType == EWorldType::Editor || ContextWorld->WorldType == EWorldType::Game;
			}
		}
	}
#endif

	return bIsActive;
}
