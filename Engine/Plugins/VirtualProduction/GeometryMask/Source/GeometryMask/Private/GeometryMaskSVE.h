// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneViewExtension.h"

class UGeometryMaskSubsystem;
class UGeometryMaskCanvas;

class FGeometryMaskSceneViewExtension
	: public FWorldSceneViewExtension
{
public:
	FGeometryMaskSceneViewExtension(const FAutoRegister& AutoRegister, UWorld* InWorld);

	// ~Begin ISceneViewExtension
	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	virtual bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext& Context) const override;
	// ~End ISceneViewExtension

private:
	TWeakObjectPtr<UGeometryMaskSubsystem> GeometryMaskSubsystemWeak;
};
