// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskSettings.h"

float UGeometryMaskSettings::GetDefaultResolutionMultiplier() const
{
	return FMath::Clamp(DefaultResolutionMultiplier, 0.25, 16.0);
}

void UGeometryMaskSettings::SetDefaultResolutionMultiplier(const float InValue)
{
	DefaultResolutionMultiplier = InValue;
}
