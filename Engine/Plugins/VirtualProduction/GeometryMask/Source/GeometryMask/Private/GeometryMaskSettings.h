// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "GeometryMaskSettings.generated.h"

UCLASS(Config = "Plugins", DefaultConfig, MinimalAPI)
class UGeometryMaskSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 DebugDF = 0;
#endif
	
public:
	virtual FName GetContainerName() const override { return FName(TEXT("Project")); }
	virtual FName GetCategoryName() const override { return FName(TEXT("Plugins")); }
	virtual FName GetSectionName() const override { return FName(TEXT("Geometry Mask")); }

#if WITH_EDITOR
	virtual FText GetSectionText() const override { return NSLOCTEXT("GeometryMaskSettings", "SectionText", "Geometry Mask"); };
	virtual FText GetSectionDescription() const override { return NSLOCTEXT("GeometryMaskSettings", "SectionDescription", "Manage Geometry Mask settings."); };
#endif

	float GetDefaultResolutionMultiplier() const;
	void SetDefaultResolutionMultiplier(const float InValue);

private:
	/** Set to enable super-sampling, ie. 2.0 will render at twice the resolution. */
	UPROPERTY(Config, EditAnywhere, Category = "Canvas", Getter, Setter, meta = (ClampMin = 0.25, ClampMax = 16.0, AllowPrivateAccess = "true"))
	float DefaultResolutionMultiplier = 1.0;
};
