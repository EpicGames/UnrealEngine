// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Level.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "GameFramework/PlayerController.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskCanvasResource.h"
#include "GeometryMaskModule.h"
#include "GeometryMaskWorldSubsystem.h"
#include "SceneView.h"
#include "UnrealClient.h"

UGeometryMaskCanvas* UGeometryMaskSubsystem::GetDefaultCanvas()
{
	return nullptr;
}

int32 UGeometryMaskSubsystem::GetNumCanvasResources() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CanvasResources_DEPRECATED.Num();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const TSet<TObjectPtr<UGeometryMaskCanvasResource>>& UGeometryMaskSubsystem::GetCanvasResources() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CanvasResources_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UGeometryMaskSubsystem::Update(UWorld* InWorld, FSceneViewFamily& InViewFamily)
{
	if (!bDoUpdates)
	{
		return;
	}

	UE_LOGF(LogGeometryMask, VeryVerbose, "UGeometryMaskSubsystem::Update World: %ls, Num. Views: %u", *InWorld->GetName(), InViewFamily.Views.Num());

	if (UGeometryMaskWorldSubsystem* Subsystem = InWorld->GetSubsystem<UGeometryMaskWorldSubsystem>())
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UGeometryMaskSubsystem::Update"), STAT_GeometryMask_UpdateAll, STATGROUP_GeometryMask);

		for (const ULevel* Level : InWorld->GetLevels())
		{
			UpdateLevel(Level, Subsystem, InViewFamily);
		}
	}
}

void UGeometryMaskSubsystem::ToggleUpdate(const TOptional<bool>& bInShouldUpdate)
{
	// New value should be user provided, or the inverse of the existing value (toggle)
	bool bShouldUpdate = bInShouldUpdate.Get(!bDoUpdates);
	if (bDoUpdates != bShouldUpdate)
	{
		bDoUpdates = bShouldUpdate;
	}
}

void UGeometryMaskSubsystem::UpdateLevel(const ULevel* InLevel, UGeometryMaskWorldSubsystem* InWorldSubsystem, FSceneViewFamily& InViewFamily)
{
	if (!IsValid(InLevel) || !InWorldSubsystem)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskSubsystem::UpdateLevel);

	int32 ViewIndex = 0;

	for (const FSceneView*& View : InViewFamily.Views)
	{
		FSceneView* MutableSceneView = const_cast<FSceneView*>(View);

		TArray<UGeometryMaskCanvasResource*, TInlineAllocator<4>> UsedResources;

		if (const FGeometryMaskLevelState* LevelState = InWorldSubsystem->FindLevelState(InLevel))
		{
			for (const TPair<FName, TObjectPtr<UGeometryMaskCanvas>>& NamedCanvas : LevelState->NamedCanvases)
			{
				NamedCanvas.Value->Update(InLevel, *MutableSceneView);

				if (UGeometryMaskCanvasResource* Resource = NamedCanvas.Value->GetResourceMutable())
				{
					UsedResources.AddUnique(Resource);
				}
			}
		}

		// Updates the texture resource
		for (UGeometryMaskCanvasResource* Resource : UsedResources)
		{
			Resource->Update(InLevel, *MutableSceneView, ViewIndex);
		}

		++ViewIndex;
	}
}

void UGeometryMaskSubsystem::AssignResourceToCanvas(UGeometryMaskCanvas* InCanvas)
{
}

void UGeometryMaskSubsystem::CompactResources()
{
}

void UGeometryMaskSubsystem::OnWorldDestroyed(UWorld* InWorld)
{
}
