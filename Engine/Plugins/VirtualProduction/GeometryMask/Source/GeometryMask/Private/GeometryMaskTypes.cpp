// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskTypes.h"

#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "UObject/UObjectThreadContext.h"

namespace UE::GeometryMask
{
	EGeometryMaskColorChannel VectorToMaskChannel(const FLinearColor& InVector)
	{
		float CurrentMax = std::numeric_limits<float>::min();
		int32 MaxIdx = 0;
		for (int32 ElementIdx = 0; ElementIdx < 4; ++ElementIdx)
		{
			const float CurrentValue = InVector.Component(ElementIdx);			
			if (CurrentValue > CurrentMax)
			{
				CurrentMax = CurrentValue;
				MaxIdx = ElementIdx;
			}
		}
		
		return static_cast<EGeometryMaskColorChannel>(MaxIdx);
	}

	EGeometryMaskColorChannel GetValidMaskChannel(
		EGeometryMaskColorChannel InColorChannel,
		bool bInIncludeAlpha)
	{
		return FMath::Clamp(InColorChannel, EGeometryMaskColorChannel::Red, bInIncludeAlpha ? EGeometryMaskColorChannel::Alpha : EGeometryMaskColorChannel::Blue);
	}

	FStringView ChannelToString(EGeometryMaskColorChannel InColorChannel)
	{
		return MaskChannelEnumToString[FMath::Clamp(InColorChannel, EGeometryMaskColorChannel::Red, EGeometryMaskColorChannel::Num)];
	}
}

uint32 GetTypeHash(const FGeometryMaskCanvasId& InCanvasId)
{
	return HashCombineFast(
		HashCombineFast(
			GetTypeHash(InCanvasId.Level),
			GetTypeHash(InCanvasId.Name)),
			InCanvasId.SceneViewIndex);
}

uint32 GetTypeHash(const FGeometryMaskDrawingContext& InUpdateContext)
{
	return HashCombineFast(GetTypeHash(InUpdateContext.Level), InUpdateContext.SceneViewIndex);
}

const FGeometryMaskCanvasId& FGeometryMaskCanvasId::None = FGeometryMaskCanvasId(EForceInit::ForceInit);
const FName FGeometryMaskCanvasId::DefaultCanvasName = TEXT("Default");

FGeometryMaskCanvasId::FGeometryMaskCanvasId(const ULevel* InLevel, const FName InName)
	: Level(InLevel)
	, Name(InName)
{
}

FGeometryMaskCanvasId::FGeometryMaskCanvasId(EForceInit)
{
	ResetToNone();
}

bool FGeometryMaskCanvasId::IsDefault() const
{
	return Name.IsEqual(DefaultCanvasName);
}

bool FGeometryMaskCanvasId::IsNone() const
{
	return Name.IsNone();
}

void FGeometryMaskCanvasId::ResetToNone()
{
	Level = nullptr;
	SceneViewIndex = 0;
	Name = NAME_None;
}

FString FGeometryMaskCanvasId::ToString() const
{
	FString LevelLabel = TEXT("(None)");
	FString WorldLabel = TEXT("(Transient)");
	FString WorldTypeLabel = TEXT("");
	if (const ULevel* ResolvedLevel = Level.ResolveObjectPtr())
	{
		if (ResolvedLevel->OwningWorld)
		{
			WorldLabel = ResolvedLevel->OwningWorld->GetName();
			WorldTypeLabel = LexToString(ResolvedLevel->OwningWorld->WorldType);
		}
		LevelLabel = ResolvedLevel->GetName();
	}

	return FString::Printf(TEXT("%s(%s - %s).%s"), *LevelLabel, *WorldLabel, *WorldTypeLabel, *Name.ToString());
}

FGeometryMaskDrawingContext::FGeometryMaskDrawingContext(TObjectKey<ULevel> InLevel, const uint8 InSceneViewIndex)
	: Level(MoveTemp(InLevel))
	, SceneViewIndex(InSceneViewIndex)
	, ViewportSize(ForceInit)
	, ViewProjectionMatrix(ForceInit)
{
}

FGeometryMaskDrawingContext::FGeometryMaskDrawingContext(const ULevel* InLevel, const uint8 InSceneViewIndex)
	: Level(InLevel)
	, SceneViewIndex(InSceneViewIndex)
	, ViewportSize(ForceInit)
	, ViewProjectionMatrix(ForceInit)
{
}

FGeometryMaskDrawingContext::FGeometryMaskDrawingContext(EForceInit)
	: Level(nullptr)
	, ViewportSize(ForceInit)
	, ViewProjectionMatrix(ForceInit)
{
}

bool FGeometryMaskDrawingContext::IsValid() const
{
	return Level.ResolveObjectPtr() != nullptr;
}

UGeometryMaskCanvasReferenceComponentBase::~UGeometryMaskCanvasReferenceComponentBase()
{
	Cleanup();
}

UCanvasRenderTarget2D* UGeometryMaskCanvasReferenceComponentBase::GetTexture() const
{
	return nullptr;
}

UTextureRenderTarget2DArray* UGeometryMaskCanvasReferenceComponentBase::GetRenderTarget() const
{
	const UGeometryMaskCanvas* Canvas = CanvasWeak.Get();
	return Canvas ? Canvas->GetRenderTarget() : nullptr;
}

void UGeometryMaskCanvasReferenceComponentBase::BeginPlay()
{
	Super::BeginPlay();

	TryResolveCanvas();
}

void UGeometryMaskCanvasReferenceComponentBase::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		TryResolveCanvas();
	}
}

void UGeometryMaskCanvasReferenceComponentBase::OnRegister()
{
	Super::OnRegister();

	if (!IsTemplate())
	{
		TryResolveCanvas();
	}
}

bool UGeometryMaskCanvasReferenceComponentBase::TryResolveNamedCanvas(FName InCanvasName)
{
	auto BroadcastSetCanvas = [this](const UGeometryMaskCanvas* InCanvas)
	{
		ReceiveSetCanvas(InCanvas);
		OnSetCanvasDelegate.Broadcast(InCanvas);
	};

	// Get current canvas
	UGeometryMaskCanvas* Canvas = CanvasWeak.Get();
	
	if (Canvas)
	{
		// Canvas set, no need to resolve it
		if (Canvas->GetFName() == InCanvasName)
		{
			return true;
		}

		// Canvas is not the correct one
		CanvasWeak.Reset();
	}

	if (UGeometryMaskWorldSubsystem* Subsystem = GetWorld()->GetSubsystem<UGeometryMaskWorldSubsystem>())
	{
		Canvas = Subsystem->GetNamedCanvas(GetComponentLevel(), InCanvasName, this);

		CanvasWeak = Canvas;
		
		if (!FUObjectThreadContext::Get().IsRoutingPostLoad)
		{
			if (Canvas)
			{
				BroadcastSetCanvas(Canvas);
			}
		}
	}

	return CanvasWeak.IsValid();
}

bool UGeometryMaskCanvasReferenceComponentBase::Cleanup()
{
	if (!GEngine)
	{
		return false;
	}

	return true;
}

bool UGeometryMaskCanvasReferenceComponentBase::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	return true;
}

