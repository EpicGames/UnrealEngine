// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskWorldSubsystem.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/World.h"
#include "GeometryMaskCanvasSharedData.h"
#include "GeometryMaskCanvasUtils.h"
#include "GeometryMaskSVE.h"
#include "IGeometryMaskClient.h"
#include "SceneViewExtension.h"

void UGeometryMaskWorldSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UWorld* World = GetWorld();

	GeometryMaskSceneViewExtension = FSceneViewExtensions::NewExtension<FGeometryMaskSceneViewExtension>(World);
}

void UGeometryMaskWorldSubsystem::Deinitialize()
{
	Super::Deinitialize();

	for (const TPair<TWeakObjectPtr<const ULevel>, FGeometryMaskLevelState>& LevelState : LevelStates)
	{
		for (const TPair<FName, TObjectPtr<UGeometryMaskCanvas>>& NamedCanvas : LevelState.Value.NamedCanvases)
		{
			if (NamedCanvas.Value)
			{
				NamedCanvas.Value->Free();
				OnGeometryMaskCanvasDestroyedDelegate.Broadcast(NamedCanvas.Value->GetCanvasId());
			}
		}
	}

	LevelStates.Empty();
}

const FGeometryMaskLevelState* UGeometryMaskWorldSubsystem::FindLevelState(const ULevel* InLevel) const
{
	return InLevel ? LevelStates.Find(InLevel) : nullptr;
}

FGeometryMaskLevelState& UGeometryMaskWorldSubsystem::FindOrAddLevelState(const ULevel* InLevel)
{
	check(IsValid(InLevel));

	FGeometryMaskLevelState* LevelState = LevelStates.Find(InLevel);
	if (!LevelState)
	{
		LevelState = &LevelStates.Add(InLevel);

		const FName ObjectName = MakeUniqueObjectName(this, UTextureRenderTarget2DArray::StaticClass(), FName(FString::Printf(TEXT("GeometryMaskRenderTarget"))));
		LevelState->RenderTarget = NewObject<UTextureRenderTarget2DArray>(this, ObjectName);

		LevelState->RenderTarget->ClearColor = FLinearColor::Black;
		LevelState->RenderTarget->bForceLinearGamma = false;
		LevelState->RenderTarget->bCanCreateUAV = true;
		LevelState->RenderTarget->bTargetArraySlicesIndependently = true;

		constexpr int32 Slices = 1;
		LevelState->RenderTarget->Init(FMath::Max(1, SharedData->TextureSize.X), FMath::Max(1, SharedData->TextureSize.Y), Slices, PF_R16F);
	}
	return *LevelState;
}

int32 FGeometryMaskLevelState::AcquireAvailableSliceIndex()
{
	if (!RenderTarget)
	{
		return INDEX_NONE;
	}

	CleanupUnusedCanvases();

	// amount of slices that should never be reached for a single level.
	constexpr int32 MaxSlices = 128;

	for (int32 SliceIndex = 0; SliceIndex < MaxSlices; ++SliceIndex)
	{
		bool bSliceIndexAvailable = true;

		for (const TWeakInterfacePtr<const IGeometryMaskClient>& MaskClientWeak : MaskClients)
		{
			if (const IGeometryMaskClient* MaskClient = MaskClientWeak.Get())
			{
				MaskClient->ForEachUsedCanvasName(
					[This=this, SliceIndex, &bSliceIndexAvailable](FName InCanvasName)
					{
						const UGeometryMaskCanvas* const Canvas = This->NamedCanvases.FindRef(InCanvasName);
						if (Canvas && SliceIndex == Canvas->GetRenderTargetSliceIndex())
						{
							bSliceIndexAvailable = false;
							return false; // break
						}
						return true; // continue
					});
			}
		}

		if (bSliceIndexAvailable)
		{
			// Update slices and render target if slice index requires a larger number of slices to be present
			if (SliceIndex >= RenderTarget->Slices)
			{
				RenderTarget->Slices = SliceIndex + 1;
				UE::GeometryMask::UpdateRenderTarget(RenderTarget);
			}
			return SliceIndex;
		}
	}

	ensureMsgf(0, TEXT("Unexpectedly found no index available below %d"), MaxSlices);
	return INDEX_NONE;
}

void FGeometryMaskLevelState::CleanupUnusedCanvases()
{
	TSet<FName> UsedCanvasNames;
	for (const TWeakInterfacePtr<const IGeometryMaskClient>& MaskClientWeak : MaskClients)
	{
		if (const IGeometryMaskClient* MaskClient = MaskClientWeak.Get())
		{
			MaskClient->ForEachUsedCanvasName(
				[&UsedCanvasNames](FName InCanvasName)
				{
					UsedCanvasNames.Add(InCanvasName);
					return true; // continue
				});
		}
	}

	for (TMap<FName, TObjectPtr<UGeometryMaskCanvas>>::TIterator Iter(NamedCanvases); Iter; ++Iter)
	{
		if (!UsedCanvasNames.Contains(Iter->Key))
		{
			if (Iter->Value)
			{
				Iter->Value->Free();
			}
			Iter.RemoveCurrent();
		}
	}
}

UGeometryMaskWorldSubsystem::UGeometryMaskWorldSubsystem()
{
	SharedData = MakeShared<FGeometryMaskCanvasSharedData>();
}

UGeometryMaskCanvas* UGeometryMaskWorldSubsystem::GetNamedCanvas(const ULevel* InLevel, FName InName)
{
	return GetNamedCanvas(InLevel, InName, /*MaskClient*/nullptr);
}

UGeometryMaskCanvas* UGeometryMaskWorldSubsystem::GetNamedCanvas(const ULevel* InLevel, FName InName, const IGeometryMaskClient* InMaskClient)
{
	if (InName.IsNone() || !InLevel)
	{
		return nullptr;
	}

	// Registers the mask client to the level state
	auto RegisterMaskClient = [InMaskClient](FGeometryMaskLevelState& InLevelState)
		{
			if (InMaskClient)
			{
				InLevelState.MaskClients.AddUnique(InMaskClient);
				InLevelState.CleanupUnusedCanvases();
			}
		};

	if (FGeometryMaskLevelState* LevelState = LevelStates.Find(InLevel))
	{
		if (UGeometryMaskCanvas* FoundCanvas = LevelState->NamedCanvases.FindRef(InName))
		{
			RegisterMaskClient(*LevelState);
			return FoundCanvas;
		}
	}

	const FName ObjectName = MakeUniqueObjectName(this, UGeometryMaskCanvas::StaticClass(), FName(FString::Printf(TEXT("GeometryMaskCanvas_%s_"), *InName.ToString())));

	FGeometryMaskLevelState& LevelState = FindOrAddLevelState(InLevel);
	RegisterMaskClient(LevelState);

	const TObjectPtr<UGeometryMaskCanvas>& NewCanvas = LevelState.NamedCanvases.Emplace(InName, NewObject<UGeometryMaskCanvas>(this, ObjectName));

	UGeometryMaskCanvas::FInitParams InitParams;
	InitParams.Level = InLevel;
	InitParams.CanvasName = InName;
	InitParams.RenderTarget = LevelState.RenderTarget;
	InitParams.SliceIndex = LevelState.AcquireAvailableSliceIndex();
	InitParams.SharedData = SharedData;

	NewCanvas->Initialize(InitParams);
	OnGeometryMaskCanvasCreatedDelegate.Broadcast(NewCanvas);

	return NewCanvas;
}

TArray<FName> UGeometryMaskWorldSubsystem::GetCanvasNames(const ULevel* InLevel)
{
	TArray<FName> CanvasNames;
	if (const FGeometryMaskLevelState* LevelState = FindLevelState(InLevel))
	{
		LevelState->NamedCanvases.GenerateKeyArray(CanvasNames);
	}
	return CanvasNames;
}

int32 UGeometryMaskWorldSubsystem::RemoveWithoutWriters()
{
	int32 NumRemoved = 0;
	
	TMap<FName, TObjectPtr<UGeometryMaskCanvas>> UsedCanvases;
	UsedCanvases.Reserve(LevelStates.Num());

	for (decltype(LevelStates)::TIterator LevelStateIter(LevelStates); LevelStateIter; ++LevelStateIter)
	{
		FGeometryMaskLevelState& LevelState = LevelStateIter.Value();

		for (decltype(LevelState.NamedCanvases)::TIterator CanvasIter(LevelState.NamedCanvases); CanvasIter; ++CanvasIter)
		{
			TObjectPtr<UGeometryMaskCanvas> NamedCanvas = CanvasIter.Value();
			if (IsValid(NamedCanvas))
			{
				if (NamedCanvas->GetWriters().IsEmpty())
				{
					OnGeometryMaskCanvasDestroyedDelegate.Broadcast(NamedCanvas->GetCanvasId());
				}

				CanvasIter.RemoveCurrent();
				++NumRemoved;
			}
		}

		if (LevelState.NamedCanvases.IsEmpty())
		{
			LevelStateIter.RemoveCurrent();
		}
	}

	return NumRemoved;
}
