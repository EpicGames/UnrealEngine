// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskWriteComponent.h"
#include "Algo/RemoveIf.h"
#include "BatchedElements.h"
#include "CanvasTypes.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/StaticMesh.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskWriter.h"
#include "GlobalRenderResources.h"
#include "StaticMeshResources.h"

UGeometryMaskWriteMeshComponent::UGeometryMaskWriteMeshComponent()
{
	MaskWriter = UE::GeometryMask::FMaskWriter::Create();
}

bool UGeometryMaskWriteMeshComponent::IsMaskWriterEnabled() const
{
	return IsRegistered();
}

void UGeometryMaskWriteMeshComponent::SetParameters(FGeometryMaskWriteParameters& InParameters)
{
	Parameters = InParameters;

	UGeometryMaskCanvas* Canvas = CanvasWeak.Get();

	// We have changed canvas, unregister this writer
	if (Canvas && Canvas->GetFName() != InParameters.CanvasName)
	{
		Canvas->RemoveWriter(this);
		CanvasWeak.Reset();
	}

	// Update canvas
	if (!CanvasWeak.IsValid())
	{
		TryResolveCanvas();
	}
}

void UGeometryMaskWriteMeshComponent::DrawToCanvas(FCanvas* InCanvas)
{
	MaskWriter->DrawToCanvas(UE::GeometryMask::FMaskWriter::FDrawParams
		{
			.Canvas = InCanvas, 
			.Actor = GetOwner(),
			.Parameters = &Parameters, 
			.bWriteWhenHidden = bWriteWhenHidden,
		});
}

UE::GeometryMask::FMaskWriter* UGeometryMaskWriteMeshComponent::GetMaskWriter()
{
	return MaskWriter.Get();
}

void UGeometryMaskWriteMeshComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	Super::OnComponentDestroyed(bDestroyingHierarchy);

	Cleanup();
}

bool UGeometryMaskWriteMeshComponent::ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const
{
	return InFunc(Parameters.CanvasName);
}

void UGeometryMaskWriteMeshComponent::UpdateCachedData()
{
}

void UGeometryMaskWriteMeshComponent::UpdateCachedStaticMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents)
{
}

void UGeometryMaskWriteMeshComponent::UpdateCachedDynamicMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents)
{
}

#if WITH_EDITOR
void UGeometryMaskWriteMeshComponent::OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent)
{
}
#endif

void UGeometryMaskWriteMeshComponent::OnDynamicMeshChanged(UDynamicMeshComponent* InDynamicMeshComponent)
{
}

bool UGeometryMaskWriteMeshComponent::TryResolveCanvas()
{
	if (TryResolveNamedCanvas(Parameters.CanvasName))
	{
		if (UGeometryMaskCanvas* Canvas = CanvasWeak.Get())
		{
			// Register this writer
			Canvas->AddWriter(this);
			return true;
		}
	}

	return false;
}

bool UGeometryMaskWriteMeshComponent::Cleanup()
{
	if (!Super::Cleanup())
	{
		return false;
	}

	if (UGeometryMaskCanvas* Canvas = CanvasWeak.Get())
	{
		Canvas->RemoveWriter(this);
		CanvasWeak.Reset();
		return true;
	}

	return false;
}

void UGeometryMaskWriteMeshComponent::ResetCachedData()
{
}
