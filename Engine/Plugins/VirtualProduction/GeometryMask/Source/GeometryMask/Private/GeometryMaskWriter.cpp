// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskWriter.h"
#include "Algo/RemoveIf.h"
#include "BatchedElements.h"
#include "CanvasTypes.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "GlobalRenderResources.h"
#include "IndexTypes.h"
#include "StaticMeshResources.h"

namespace UE::GeometryMask
{

TSharedRef<FMaskWriter> FMaskWriter::Create()
{
	return MakeShared<FMaskWriter>();
}

void FMaskWriter::DrawToCanvas(const FDrawParams& InParams)
{
	if (!InParams.Canvas || (!InParams.bWriteWhenHidden && InParams.Actor && InParams.Actor->IsHidden()))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryMaskWriter::DrawToCanvas);
	UpdateCachedData(InParams.Actor);

	const FLinearColor Color = InParams.Parameters->OperationType == EGeometryMaskCompositeOperation::Subtract 
		? FLinearColor::Black 
		: FLinearColor::White;

	constexpr ESimpleElementBlendMode ElementBlendMode = ESimpleElementBlendMode::SE_BLEND_Opaque;
	const FHitProxyId CanvasHitProxyId = InParams.Canvas->GetHitProxyId();

	for (TMap<TObjectKey<UPrimitiveComponent>, FObjectKey>::TIterator Iter(CachedComponents); Iter; ++Iter)
	{
		if (!CachedMeshData.Contains(Iter->Value))
		{
			Iter.RemoveCurrent();
			continue;
		}

		const UPrimitiveComponent* const Component = Iter->Key.ResolveObjectPtr();
		if (!Component)
		{
			Iter.RemoveCurrent();
			continue;
		}

		const FTransform& LocalToWorld = Component->GetComponentToWorld();
		const FGeometryMaskBatchElementData& MeshBatchElementData = CachedMeshData[Iter->Value];

		InParams.Canvas->PushAbsoluteTransform(LocalToWorld.ToMatrixWithScale());
		ON_SCOPE_EXIT
		{
			InParams.Canvas->PopTransform();
		};

		FBatchedElements* const CanvasTriangleBatchedElements = InParams.Canvas->GetBatchedElements(FCanvas::ET_Triangle);
		CanvasTriangleBatchedElements->AddReserveVertices(MeshBatchElementData.Vertices.Num());
		CanvasTriangleBatchedElements->AddReserveTriangles(MeshBatchElementData.NumTriangles, GWhiteTexture, ElementBlendMode);

		auto AddVertex = [CanvasTriangleBatchedElements, &MeshBatchElementData, CanvasHitProxyId, &Color](int32 VertexIdx) -> int32
			{
				return CanvasTriangleBatchedElements->AddVertexf(
					MeshBatchElementData.Vertices[VertexIdx],
					FVector2f::ZeroVector,
					Color,
					CanvasHitProxyId);
			};

		// Store the index of the first vertex to be able to convert the MeshBatchElementData vertex indices to
		// map to the actual vertices added to the BatchedElements.
		int32 InitialIndex = 0;

		if (!MeshBatchElementData.Vertices.IsEmpty())
		{
			InitialIndex = AddVertex(0);
			for (int32 VertexIdx = 1; VertexIdx < MeshBatchElementData.Vertices.Num(); ++VertexIdx)
			{
				AddVertex(VertexIdx);
			}
		}

		for (int32 VertexIdx = 0; VertexIdx <= MeshBatchElementData.Indices.Num() - 3; VertexIdx += 3)
		{
			// MeshBatchElementData.Indices are 'relative' in the sense that do not consider possible existing 
			// vertices within BatchedElements prior to its vertices being added.
			// So these 'local' indices are converted to be the proper indices of the vertices that were added above. 
			const int32 V0 = InitialIndex + MeshBatchElementData.Indices[VertexIdx];
			const int32 V1 = InitialIndex + MeshBatchElementData.Indices[VertexIdx + 1];
			const int32 V2 = InitialIndex + MeshBatchElementData.Indices[VertexIdx + 2];

			CanvasTriangleBatchedElements->AddTriangle(V0, V1, V2, GWhiteTexture, ElementBlendMode);
		}
	}
}

void FMaskWriter::ForEachMaskPrimitive(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const
{
	for (const TPair<TObjectKey<UPrimitiveComponent>, FObjectKey>& Pair : CachedComponents)
	{
		if (UPrimitiveComponent* Component = Pair.Key.ResolveObjectPtr())
		{
			InFunc(Component);
		}
	}
}

void FMaskWriter::ResetCachedData()
{
	LastPrimitiveComponentCountMap.Reset();
	CachedComponents.Reset();
	CachedMeshData.Reset();
}

void FMaskWriter::UpdateCachedData(AActor* InActor)
{
	if (InActor)
	{
		TArray<UPrimitiveComponent*> PrimitiveComponents;
		InActor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);

		// @todo: better check for cached data state?
		int32& LastPrimitiveComponentCount = LastPrimitiveComponentCountMap.FindOrAdd(InActor, 0);
		if (LastPrimitiveComponentCount == PrimitiveComponents.Num())
		{
			return;
		}

		LastPrimitiveComponentCount = PrimitiveComponents.Num();
		UpdateCachedStaticMeshData(PrimitiveComponents);
		UpdateCachedDynamicMeshData(PrimitiveComponents);
	}
}

void FMaskWriter::UpdateCachedStaticMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents)
{
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	StaticMeshComponents.Reserve(InPrimitiveComponents.Num());

	Algo::TransformIf(
		InPrimitiveComponents,
		StaticMeshComponents,
		[](const UPrimitiveComponent* InComponent)
		{
			return InComponent->IsA<UStaticMeshComponent>();
		},
		[](UPrimitiveComponent* InComponent)
		{
			return Cast<UStaticMeshComponent>(InComponent);
		});

	if (StaticMeshComponents.IsEmpty())
	{
		return;
	}

	// Remove built/already cached
	StaticMeshComponents.SetNum(Algo::RemoveIf(StaticMeshComponents, [this](const UStaticMeshComponent* InStaticMeshComponent)
	{
		const UStaticMesh* StaticMesh = InStaticMeshComponent->GetStaticMesh();
	
		if (!StaticMesh || !StaticMesh->HasValidRenderData())
		{
			return true;
		}

		if (CachedComponents.Contains(InStaticMeshComponent) && CachedMeshData.Contains(StaticMesh))
		{
			return true;
		}

		return false;
	}));

	// Subscribe to change events
#if WITH_EDITOR
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		StaticMeshComponent->OnStaticMeshChanged().RemoveAll(this);
		StaticMeshComponent->OnStaticMeshChanged().AddSPLambda(this, 
			[this](UStaticMeshComponent* InStaticMeshComponent)
			{
				OnStaticMeshChanged(InStaticMeshComponent);
			});
	}
#endif

	// Used for cache lookup
	TArray<FObjectKey> StaticMeshObjects;
	StaticMeshObjects.Reserve(StaticMeshComponents.Num());

	TArray<FStaticMeshLODResources*> StaticMeshResources;
	StaticMeshResources.Reserve(StaticMeshComponents.Num());

	// Collect valid mesh resources
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::CollectMeshResources);
		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
			{
				if (!StaticMesh->HasValidRenderData())
				{
					continue;
				}
			
				if (FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData())
				{
					if (RenderData->LODResources.IsEmpty())
					{
						continue;
					}

					if (RenderData->LODResources[0].GetNumVertices() == 0)
					{
						continue;
					}

					StaticMeshObjects.Add(StaticMesh);
					StaticMeshResources.Add(&RenderData->LODResources[0]);
					CachedComponents.Add(StaticMeshComponent, StaticMesh);
				}
			}
		}

		// Convert mesh resources to batch elements
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::ConvertMeshResources);
			
			CachedMeshData.Reserve(CachedMeshData.Num() + StaticMeshObjects.Num());

			for (int32 MeshIdx = 0; MeshIdx < StaticMeshResources.Num(); ++MeshIdx)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::ConvertMeshResources::Task);

				FStaticMeshLODResources* MeshResources = StaticMeshResources[MeshIdx];
				const int32 NumVertices = MeshResources->GetNumVertices();
				const int32 NumIndices = MeshResources->IndexBuffer.GetNumIndices();
				const int32 NumTriangles = MeshResources->GetNumTriangles();

				FGeometryMaskBatchElementData& MeshBatchElementData = CachedMeshData.Emplace(StaticMeshObjects[MeshIdx]);
				MeshBatchElementData.Reserve(NumVertices, NumIndices, NumTriangles);
				MeshResources->IndexBuffer.GetCopy(MeshBatchElementData.Indices);

				for (int32 VertexIdx = 0; VertexIdx < NumVertices; ++VertexIdx)
				{
					MeshBatchElementData.Vertices.Add(FVector4f(MeshResources->VertexBuffers.PositionVertexBuffer.VertexPosition(VertexIdx)));
				}
			}
		}
	}
}

void FMaskWriter::UpdateCachedDynamicMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents)
{
	TArray<UDynamicMeshComponent*> DynamicMeshComponents;
	DynamicMeshComponents.Reserve(InPrimitiveComponents.Num());
	
	Algo::TransformIf(
		InPrimitiveComponents,
		DynamicMeshComponents,
		[](const UPrimitiveComponent* InComponent)
		{
			return InComponent->IsA<UDynamicMeshComponent>();
		},
		[](UPrimitiveComponent* InComponent)
		{
			return Cast<UDynamicMeshComponent>(InComponent);
		});

	if (DynamicMeshComponents.IsEmpty())
	{
		return;
	}

	// Remove built/already cached
	DynamicMeshComponents.SetNum(Algo::RemoveIf(DynamicMeshComponents, [this](UDynamicMeshComponent* InDynamicMeshComponent)
	{
		const UDynamicMesh* DynamicMesh = InDynamicMeshComponent->GetDynamicMesh();
	
		if (!DynamicMesh || !DynamicMesh->GetMeshPtr())
		{
			return true;
		}

		if (CachedComponents.Contains(InDynamicMeshComponent) && CachedMeshData.Contains(DynamicMesh))
		{
			return true;
		}

		return false;
	}));

	// Subscribe to change events
	for (UDynamicMeshComponent* DynamicMeshComponent : DynamicMeshComponents)
	{
		if (FDynamicMesh3* DynamicMesh3 = DynamicMeshComponent->GetMesh())
		{
			DynamicMesh3->SetShapeChangeStampEnabled(true);
		}

		DynamicMeshComponent->OnMeshChanged.RemoveAll(this);
		DynamicMeshComponent->OnMeshChanged.AddSPLambda(this, 
			[this, DynamicMeshComponent]()
			{
				OnDynamicMeshChanged(DynamicMeshComponent);
			});
	}

	// Used for cache lookup
	TArray<FObjectKey> DynamicMeshObjects;
	DynamicMeshObjects.Reserve(DynamicMeshComponents.Num());
	
	TArray<FDynamicMesh3*> DynamicMeshes;
	DynamicMeshes.Reserve(DynamicMeshComponents.Num());

	// Collect valid meshes
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::CollectMeshResources);
		for (UDynamicMeshComponent* DynamicMeshComponent : DynamicMeshComponents)
		{
			if (UDynamicMesh* DynamicMeshObject = DynamicMeshComponent->GetDynamicMesh())
			{
				if (FDynamicMesh3* DynamicMesh = DynamicMeshObject->GetMeshPtr())
				{
					if (DynamicMesh->TriangleCount() == 0)
					{
						continue;
					}

					DynamicMeshObjects.Add(DynamicMeshObject);
					DynamicMeshes.Add(DynamicMesh);
					CachedComponents.Add(DynamicMeshComponent, DynamicMeshObject);
				}
			}
		}
	}

	// Convert meshes to batch elements
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::ConvertMeshResources);
		
		CachedMeshData.Reserve(CachedMeshData.Num() + DynamicMeshObjects.Num());
		
		for (int32 MeshIdx = 0; MeshIdx < DynamicMeshes.Num(); ++MeshIdx)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UGeometryMaskWriteMeshComponent::DrawToCanvas::ConvertMeshResources::Task);

			FDynamicMesh3 CompactMesh;
			const FDynamicMesh3* DynamicMesh = DynamicMeshes[MeshIdx];
			CompactMesh.CompactCopy(*DynamicMesh);

			const int32 NumVertices = CompactMesh.VertexCount();
			const int32 NumIndices = CompactMesh.TriangleCount() * 3;
			const int32 NumTriangles = CompactMesh.TriangleCount();

			FGeometryMaskBatchElementData& MeshBatchElementData = CachedMeshData.Emplace(DynamicMeshObjects[MeshIdx]);
			MeshBatchElementData.ChangeStamp = DynamicMesh->GetChangeStamp();
			MeshBatchElementData.Reserve(NumVertices, NumIndices, NumTriangles);

			for (const int32 VertexIdx : CompactMesh.VertexIndicesItr())
			{
				MeshBatchElementData.Vertices.Add(FVector4f(FVector3f(CompactMesh.GetVertex(VertexIdx))));				
			}

			for (const UE::Geometry::FIndex3i Triangle : CompactMesh.TrianglesItr())
			{
				MeshBatchElementData.Indices.Append({
					static_cast<uint32>(Triangle.A),
					static_cast<uint32>(Triangle.B),
					static_cast<uint32>(Triangle.C)});
			}
		}
	}
}

#if WITH_EDITOR
void FMaskWriter::OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent)
{
	// Triggers a cache refresh
	ResetCachedData();
}
#endif

void FMaskWriter::OnDynamicMeshChanged(UDynamicMeshComponent* InDynamicMeshComponent)
{
	// Triggers a specific cache refresh
	if (FGeometryMaskBatchElementData* CachedData = CachedMeshData.Find(InDynamicMeshComponent->GetDynamicMesh()))
	{
		CachedData->ChangeStamp = INDEX_NONE;
	}
	// Triggers a general cache refresh
	ResetCachedData();
}

} // UE::GeometryMask
