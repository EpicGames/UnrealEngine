// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHICommandList.h"

class FRenderTarget;

class IGeometryMaskPostProcess
{
public:
	virtual ~IGeometryMaskPostProcess() = default;
	
	virtual void Execute(FRenderTarget* InTexture) = 0;

protected:
	virtual void Execute_RenderThread(FRHICommandListImmediate& InRHICmdList, FRenderTarget* InTexture) = 0;
};

template <typename InParametersType>
class TGeometryMaskPostProcess
	: public IGeometryMaskPostProcess
{
public:
	explicit TGeometryMaskPostProcess(const InParametersType& InParameters)
		: Parameters(InParameters) { }

	const InParametersType& GetParameters() const
	{
		return Parameters;
	}
	
	void SetParameters(const InParametersType& InParameters)
	{
		Parameters = InParameters;
	}

protected:
	InParametersType Parameters;
};

