// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskPostProcess_Blur.h"

#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "UnrealClient.h"

namespace UE::GeometryMask::Private
{
	static constexpr int32 MaxNumSteps = 127;
	static constexpr int32 WeightsAndOffsetsMaxNum = 64; // Half of Num Steps
}

class FGeometryMaskBlurCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FGeometryMaskBlurCS);
	SHADER_USE_PARAMETER_STRUCT(FGeometryMaskBlurCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, SampleCount)
		SHADER_PARAMETER_ARRAY(FVector4f, WeightsAndOffsets, [UE::GeometryMask::Private::WeightsAndOffsetsMaxNum])
		SHADER_PARAMETER(FVector4f, BufferSizeAndDirection)
		SHADER_PARAMETER(FIntPoint, InputDimensions)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture3D<float4>, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

IMPLEMENT_GLOBAL_SHADER(FGeometryMaskBlurCS, "/Plugin/GeometryMask/Private/GeometryMaskBlurCS.usf", "MainCS", SF_Compute);

namespace UE::GeometryMask
{
	namespace Private
	{
		static float GetWeight(float InDistance, float InStrength)
		{
			if (FMath::IsNearlyZero(InStrength))
			{
				return 0.0f;
			}

			// from https://en.wikipedia.org/wiki/Gaussian_blur
			const float Strength2 = InStrength * InStrength;
			return (1.0f / FMath::Sqrt(2 * PI * Strength2)) * FMath::Exp(-(InDistance * InDistance) / (2 * Strength2));
		}

		static FVector2f GetWeightAndOffset(float InDistance, float InSigma)
		{
			const float Offset1 = InDistance;
			const float Weight1 = GetWeight(Offset1, InSigma);

			const float Offset2 = InDistance + 1;
			const float Weight2 = GetWeight(Offset2, InSigma);

			const float TotalWeight = Weight1 + Weight2;

			float Offset = 0;
			if (TotalWeight > 0)
			{
				Offset = (Weight1*Offset1 + Weight2*Offset2) / TotalWeight;
			}

			return FVector2f(TotalWeight, Offset);
		}

		static int32 ComputeWeights(int32 InKernelSize, float InSigma, TArray<FVector4f>& OutWeightsAndOffsets)
		{
			const int32 NumSamples = FMath::DivideAndRoundUp(InKernelSize, 2);

			// We need half of the sample count array because we're packing two samples into one float4
			OutWeightsAndOffsets.AddUninitialized(NumSamples % 2 == 0 ? NumSamples / 2 : NumSamples / 2 + 1);

			OutWeightsAndOffsets[0] = FVector4f(FVector2f(GetWeight(0,InSigma), 0), GetWeightAndOffset(1, InSigma));
			int32 SampleIndex = 1;
			for (int32 X = 3; X < InKernelSize; X += 4)
			{
				OutWeightsAndOffsets[SampleIndex] = FVector4f(GetWeightAndOffset((float)X, InSigma), GetWeightAndOffset((float)(X + 2), InSigma));
				++SampleIndex;
			}

			return NumSamples;
		}

		int32 ComputeBlurWeights(int32 InKernelSize, float InStdDev, TArray<FVector4f>& OutWeightsAndOffsets)
		{
			return ComputeWeights(InKernelSize, InStdDev, OutWeightsAndOffsets);
		}
	}

	namespace Internal
	{
		// @see: SBackgroundBlur::ComputeEffectiveKernelSize
		int32 ComputeEffectiveKernelSize(double InStrength)
		{
			// If the radius isn't set, auto-compute it based on the strength
			// @note: * 4.0 differs from * 0.3 in the slate code as it seems to produce better auto-computed results
			int32 KernelSize = FMath::RoundToInt(InStrength * 4.0f);
		
			if (KernelSize % 2 == 0)
			{
				++KernelSize;
			}

			static constexpr int32 MaxKernelSize = 255;
			KernelSize = FMath::Clamp(KernelSize, 3, MaxKernelSize);

			return KernelSize;
		}
		
		// @see: SBackgroundBlur::ComputeEffectiveKernelSize
		int32 ComputeEffectiveKernelSize2(double InStrength)
		{
			// If the radius isn't set, auto-compute it based on the strength
			// @note: * 4.0 differs from * 0.3 in the slate code as it seems to produce better auto-computed results
			int32 KernelSize = FMath::RoundToInt((InStrength * 2) + 1);

			if (KernelSize % 2 == 0)
			{
				++KernelSize;
			}

			static constexpr int32 MinKernelSize = 3;
			static constexpr int32 MaxKernelSize = 255;
			KernelSize = FMath::Clamp(KernelSize, MinKernelSize, MaxKernelSize);

			return KernelSize;
		}
	}
}

FGeometryMaskPostProcess_Blur::FGeometryMaskPostProcess_Blur(const FGeometryMaskPostProcessParameters_Blur& InParameters)
	: Super(InParameters)
{
}

void FGeometryMaskPostProcess_Blur::Execute(FRenderTarget* InTexture)
{
	// If blur NOT applied to any channel, early-out 
	if (!InTexture || !Parameters.bApplyBlur)
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(CanvasRenderTargetMakeCurrentCommand)(
		[Self = AsShared(), InTexture](FRHICommandListImmediate& InRHICmdList)
		{
			Self->Execute_RenderThread(InRHICmdList, InTexture);
		});
}

void FGeometryMaskPostProcess_Blur::Execute_RenderThread(FRHICommandListImmediate& InRHICmdList, FRenderTarget* InTexture)
{
	ensure(IsInRenderingThread());

	TArray<FVector4f> WeightsAndOffsets;

	const int32 BlurKernelSize = UE::GeometryMask::Internal::ComputeEffectiveKernelSize(Parameters.BlurStrength);
	const double BlurStrength = FMath::Max(0.5, Parameters.BlurStrength);
	const int32 SampleCount = UE::GeometryMask::Private::ComputeBlurWeights(BlurKernelSize, BlurStrength, WeightsAndOffsets);

	FRDGBuilder GraphBuilder(InRHICmdList);
	{
		FIntPoint InputSize = InTexture->GetSizeXY();

		FRDGTextureRef InputTexture = InTexture->GetRenderTargetTexture(GraphBuilder);
		FRDGTextureSRVRef InputTexture_SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(InputTexture, Parameters.SliceIndex));
		
		FRDGTextureDesc IntermediateTextureDesc(
			FRDGTextureDesc::Create2D(
				InputTexture->Desc.Extent,
				InputTexture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV));

		FRDGTextureRef IntermediateTexture = GraphBuilder.CreateTexture(IntermediateTextureDesc, TEXT("GeometryMaskBlur.IntermediateTexture"));
		FRDGTextureSRVRef IntermediateTexture_SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc(IntermediateTexture));
		FRDGTextureUAVRef IntermediateTexture_UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntermediateTexture));

		FRDGTextureDesc OutputTextureDesc(
			FRDGTextureDesc::Create2D(
				InputTexture->Desc.Extent,
				InputTexture->Desc.Format,
				FClearValueBinding::None,
				TexCreate_ShaderResource | TexCreate_UAV));

		FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(OutputTextureDesc, TEXT("GeometryMaskBlur.OutputTexture"));
		FRDGTextureUAVRef OutputTexture_UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(OutputTexture));

		TShaderMapRef<FGeometryMaskBlurCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		const FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(InputSize.Size(), FIntPoint(8, 8));

		// Horizontal Blur
		{
			RDG_EVENT_SCOPE(GraphBuilder, "GeometryMaskBlurH");
			
			FGeometryMaskBlurCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeometryMaskBlurCS::FParameters>();
			{
				const int32 WeightsAndOffsetsNum = FMath::Min(WeightsAndOffsets.Num(), PassParameters->WeightsAndOffsets.Num());
				for (int32 Index = 0; Index < WeightsAndOffsetsNum; ++Index)
				{
					PassParameters->WeightsAndOffsets[Index] = WeightsAndOffsets[Index];
				}
				PassParameters->BufferSizeAndDirection = FVector4f(1.0f / InputSize.X, 1.0f / InputSize.Y, 1.0f, 0.0f);
				PassParameters->SampleCount = SampleCount;
				PassParameters->InputDimensions = InputSize;
				PassParameters->InputTexture = InputTexture_SRV;
				PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				PassParameters->OutputTexture = IntermediateTexture_UAV;
			}

			ClearUnusedGraphResources(ComputeShader, PassParameters);
			
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("BlurH"),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				NumGroups);
		}

		// Vertical Blur
		{
			RDG_EVENT_SCOPE(GraphBuilder, "GeometryMaskBlurV");
			
			FGeometryMaskBlurCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeometryMaskBlurCS::FParameters>();
			{
				const int32 WeightsAndOffsetsNum = FMath::Min(WeightsAndOffsets.Num(), PassParameters->WeightsAndOffsets.Num());
				for (int32 Index = 0; Index < WeightsAndOffsetsNum; ++Index)
				{
					PassParameters->WeightsAndOffsets[Index] = WeightsAndOffsets[Index];
				}
				PassParameters->BufferSizeAndDirection = FVector4f(1.0f / InputSize.X, 1.0f / InputSize.Y, 0.0f, 1.0f);
				PassParameters->SampleCount = SampleCount;
				PassParameters->InputDimensions = InputSize;
				PassParameters->InputTexture = IntermediateTexture_SRV;
				PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
				PassParameters->OutputTexture = OutputTexture_UAV;
			}

			ClearUnusedGraphResources(ComputeShader, PassParameters);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("BlurV"),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				NumGroups);
		}

		// Copy back to RenderTarget
		{
			RDG_EVENT_SCOPE(GraphBuilder, "GeometryMaskBlurPostCopy");

			FRHICopyTextureInfo CopyTextureInfo;
			CopyTextureInfo.SourceSliceIndex = 0;
			CopyTextureInfo.DestSliceIndex = Parameters.SliceIndex; // the dest target is the input texture
			CopyTextureInfo.NumSlices = 1; // only one slice to copy

			AddCopyTexturePass(GraphBuilder, OutputTexture, InputTexture, CopyTextureInfo);
		}
	}

	GraphBuilder.Execute();
}
