// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "GeometryMaskConstants.h"
#include "GeometryMaskPostProcess.h"
#include "Templates/SharedPointer.h"

namespace UE::GeometryMask::Internal
{
	int32 ComputeEffectiveKernelSize(double InStrength);
}

struct FGeometryMaskPostProcessParameters_Blur
{
	bool bApplyBlur = false;
	uint16 SliceIndex = 0;
	double BlurStrength = 16.0;
};

class FGeometryMaskPostProcess_Blur
	: public TGeometryMaskPostProcess<FGeometryMaskPostProcessParameters_Blur>
	, public TSharedFromThis<FGeometryMaskPostProcess_Blur>
{
	using Super = TGeometryMaskPostProcess<FGeometryMaskPostProcessParameters_Blur>;
	
public:
	explicit FGeometryMaskPostProcess_Blur(const FGeometryMaskPostProcessParameters_Blur& InParameters);

	virtual void Execute(FRenderTarget* InTexture) override;

protected:
	virtual void Execute_RenderThread(FRHICommandListImmediate& InRHICmdList, FRenderTarget* InTexture) override;
};
