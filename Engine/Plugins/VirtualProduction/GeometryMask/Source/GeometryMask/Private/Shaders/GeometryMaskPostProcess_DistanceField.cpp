// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskPostProcess_DistanceField.h"

#include "GeometryMaskModule.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RHIStaticStates.h"
#include "UnrealClient.h"

DECLARE_GPU_STAT_NAMED(GeometryMaskJFInit, TEXT("GeometryMaskJFInit"));
DECLARE_GPU_STAT_NAMED(GeometryMaskJFStep, TEXT("GeometryMaskJFStep"));
DECLARE_GPU_STAT_NAMED(GeometryMaskJFtoDF, TEXT("GeometryMaskJFtoDF"));

namespace UE::GeometryMask::Private
{
	static constexpr int32 NumNeighbors = 8;
	static constexpr int32 MaxSteps = 13;

	static int32 CalculateStepCount(double InRadiusSize)
	{
		return InRadiusSize == 0 ? 0 : FMath::CeilToInt32(FMath::Log2(InRadiusSize)) - 1;
	}

	static uint32 CalculateStepSize(int32 InStepIdx, int32 InMaxSteps)
	{
		int32 StepDelta = InMaxSteps - InStepIdx;
		StepDelta = FMath::Min(InMaxSteps, StepDelta);
		return 1 << StepDelta;
	}
}

class FGeometryMaskJFInitCSBase : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FGeometryMaskJFInitCSBase, NonVirtual);
	SHADER_USE_PARAMETER_STRUCT(FGeometryMaskJFInitCSBase, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, InputDimensions)
		SHADER_PARAMETER(FVector2f, OneOverInputDimensions)
		SHADER_PARAMETER(FVector2f, UVRatioAdjustment)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, OutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

template <int32 NumChannels>
class TGeometryMaskJFInitCS : public FGeometryMaskJFInitCSBase
{
	DECLARE_GLOBAL_SHADER(TGeometryMaskJFInitCS);
	SHADER_USE_PARAMETER_STRUCT(TGeometryMaskJFInitCS, FGeometryMaskJFInitCSBase);

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("KERNEL_SIZE"), 3);
	}
};

using FGeometryMaskJFInitCS1 = TGeometryMaskJFInitCS<1>;
IMPLEMENT_SHADER_TYPE(template<>, FGeometryMaskJFInitCS1, TEXT("/Plugin/GeometryMask/Private/GeometryMaskJFInitCS.usf"), TEXT("MainCS"), SF_Compute);

class FGeometryMaskJFStepCSBase : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FGeometryMaskJFStepCSBase, NonVirtual);
	SHADER_USE_PARAMETER_STRUCT(FGeometryMaskJFStepCSBase, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, StepSize)
		SHADER_PARAMETER(FIntPoint, InputDimensions)
		SHADER_PARAMETER(FVector2f, OneOverInputDimensions)
		SHADER_PARAMETER(FVector2f, UVRatioAdjustment)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, InputBuffer)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, OutputBuffer)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), FComputeShaderUtils::kGolden2DGroupSize);
		OutEnvironment.SetDefine(TEXT("MAX_STEPS"), UE::GeometryMask::Private::MaxSteps);
		OutEnvironment.SetDefine(TEXT("KERNEL_SIZE"), 3);
	}
};

template <int32 NumChannels>
class TGeometryMaskJFStepCS : public FGeometryMaskJFStepCSBase
{
	DECLARE_GLOBAL_SHADER(TGeometryMaskJFStepCS);
	SHADER_USE_PARAMETER_STRUCT(TGeometryMaskJFStepCS, FGeometryMaskJFStepCSBase);

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGeometryMaskJFStepCSBase::ModifyCompilationEnvironment(InParameters, OutEnvironment);
	}
};

using FGeometryMaskJFStepCS1 = TGeometryMaskJFStepCS<1>;
IMPLEMENT_SHADER_TYPE(template<>, FGeometryMaskJFStepCS1, TEXT("/Plugin/GeometryMask/Private/GeometryMaskJFStepCS.usf"), TEXT("MainCS"), SF_Compute);

class FGeometryMaskJFtoDFCSBase : public FGlobalShader
{
	DECLARE_INLINE_TYPE_LAYOUT(FGeometryMaskJFtoDFCSBase, NonVirtual);
	SHADER_USE_PARAMETER_STRUCT(FGeometryMaskJFtoDFCSBase, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntPoint, InputDimensions)
		SHADER_PARAMETER(FVector2f, OneOverInputDimensions)
		SHADER_PARAMETER(FVector2f, UVRatioAdjustment)
		SHADER_PARAMETER(float, StepDistanceMultiplier)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, OriginalInputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, OriginalInputSampler)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, InputBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, OutputTexture)
	END_SHADER_PARAMETER_STRUCT()

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("TILE_SIZE"), FComputeShaderUtils::kGolden2DGroupSize);
	}
};

template <int32 NumChannels>
class TGeometryMaskJFtoDFCS : public FGeometryMaskJFtoDFCSBase
{
	DECLARE_GLOBAL_SHADER(TGeometryMaskJFtoDFCS);
	SHADER_USE_PARAMETER_STRUCT(TGeometryMaskJFtoDFCS, FGeometryMaskJFtoDFCSBase);

public:
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGeometryMaskJFtoDFCSBase::ModifyCompilationEnvironment(InParameters, OutEnvironment);

		OutEnvironment.SetDefine(TEXT("NUM_CHANNELS"), NumChannels);
	}
};

using FGeometryMaskJFtoDFCS1 = TGeometryMaskJFtoDFCS<1>;
IMPLEMENT_SHADER_TYPE(template<>, FGeometryMaskJFtoDFCS1, TEXT("/Plugin/GeometryMask/Private/GeometryMaskJFtoDFCS.usf"), TEXT("MainCS"), SF_Compute);

FGeometryMaskPostProcess_DistanceField::FGeometryMaskPostProcess_DistanceField(const FGeometryMaskPostProcessParameters_DistanceField& InParameters)
	: Super(InParameters)
{
}

void FGeometryMaskPostProcess_DistanceField::Execute(FRenderTarget* InTexture)
{
	if (!InTexture)
	{
		return;
	}

	// If blur NOT applied to any channel, early-out
	if (!Parameters.bCalculateDF)
	{
		LastInputSize.Reset();
		return;
	}

	ENQUEUE_RENDER_COMMAND(CanvasRenderTargetMakeCurrentCommand)(
		[Self = AsShared(), InTexture](FRHICommandListImmediate& InRHICmdList)
		{
			Self->Execute_RenderThread(InRHICmdList, InTexture);
		});
}

void FGeometryMaskPostProcess_DistanceField::Execute_RenderThread(
	FRHICommandListImmediate& InRHICmdList,
	FRenderTarget* InTexture)
{
	ensure(IsInRenderingThread());

	uint8 DebugPass = 0; // 0 = no debug
	
#if WITH_EDITORONLY_DATA
	DebugPass = GetDefault<UGeometryMaskSettings>()->DebugDF;
#endif

	FRDGBuilder GraphBuilder(InRHICmdList);
	{
		FIntPoint InputSize = InTexture->GetSizeXY();
		FVector2f OneOverInputSize = FVector2f::One() / InputSize;

		const bool bReallocBuffer = !LastInputSize.IsSet() 
			|| *LastInputSize != InputSize
#if WITH_EDITOR
			|| DebugPass > 0
#endif
			|| !StoredInitOutputBuffer.IsValid();

		LastInputSize = InputSize;

		// Scales UV's such that the X axis is always 1.0
		FVector2f InputHeightOverWidth = FVector2f(1.0f, static_cast<float>(InputSize.Y) / static_cast<float>(InputSize.X));

		FIntVector NumGroups = FComputeShaderUtils::GetGroupCount(InputSize, FIntPoint(8, 8));

		const FIntPoint BufferSize = InputSize;
		const uint32 BufferLength = BufferSize.X * BufferSize.Y;

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		TShaderRef<FGeometryMaskJFInitCSBase> InitComputeShader;
		TShaderRef<FGeometryMaskJFStepCSBase> StepComputeShader;
		TShaderRef<FGeometryMaskJFtoDFCSBase> OutputComputeShader;

		{
			TShaderMapRef<FGeometryMaskJFInitCS1> InitComputeShader1(GlobalShaderMap);
			TShaderMapRef<FGeometryMaskJFStepCS1> StepComputeShader1(GlobalShaderMap);
			TShaderMapRef<FGeometryMaskJFtoDFCS1> OutputComputeShader1(GlobalShaderMap);

			InitComputeShader = InitComputeShader1;
			StepComputeShader = StepComputeShader1;
			OutputComputeShader = OutputComputeShader1;
		}

		FRDGTextureRef InputTexture = InTexture->GetRenderTargetTexture(GraphBuilder);
		FRDGTextureSRVRef InputTexture_SRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForSlice(InputTexture, Parameters.SliceIndex));
		FRDGTextureUAVRef InputTexture_UAV = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(InputTexture, /*MipLevel*/0, /*Format*/InputTexture->Desc.Format, Parameters.SliceIndex, /*NumSlices*/1));

		FVector4f* BufferData = nullptr;
		if (bReallocBuffer)
		{
			BufferData = GraphBuilder.AllocPODArray<FVector4f>(BufferLength);
		}

		FRDGBufferRef InitOutputBuffer = nullptr;
		if (bReallocBuffer)
		{
			FRDGBufferDesc InitOutputBufferDesc =
				FRDGBufferDesc::CreateBufferDesc(
					sizeof(FVector4f),
					BufferLength);
			InitOutputBufferDesc.Usage = EBufferUsageFlags::UnorderedAccess;

			InitOutputBuffer = GraphBuilder.CreateBuffer(InitOutputBufferDesc, TEXT("GeometryMaskJFInit.OutputBuffer"));
			StoredInitOutputBuffer = GraphBuilder.ConvertToExternalBuffer(InitOutputBuffer);

			GraphBuilder.QueueBufferUpload(InitOutputBuffer, BufferData, sizeof(FVector4f) * BufferLength, ERDGInitialDataFlags::NoCopy);
		}
		else
		{
			InitOutputBuffer = GraphBuilder.RegisterExternalBuffer(StoredInitOutputBuffer, TEXT("GeometryMaskJFInit.OutputBuffer"));
		}

		const EPixelFormat BufferFormat = EPixelFormat::PF_FloatRGBA;

		// 1. Init from input binary-ish texture
		{
			FRDGBufferUAVRef OutputBuffer_UAV = GraphBuilder.CreateUAV(InitOutputBuffer, BufferFormat);

			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, GeometryMaskJFInit, "GeometryMaskJFInit");

				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryMaskJFInit);
				DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FGeometryMaskPostProcess_DistanceField::GeometryMaskJFInit"), STAT_GeometryMask_GeometryMaskJFInit, STATGROUP_GeometryMask);

				FGeometryMaskJFInitCSBase::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeometryMaskJFInitCSBase::FParameters>();
				{
					PassParameters->InputDimensions = InputSize;
					PassParameters->OneOverInputDimensions = OneOverInputSize;
					PassParameters->UVRatioAdjustment = InputHeightOverWidth;
					PassParameters->InputTexture = InputTexture_SRV;
					PassParameters->InputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
					PassParameters->OutputBuffer = OutputBuffer_UAV;
				}

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("Init"),
					ERDGPassFlags::Compute,
					InitComputeShader,
					PassParameters,
					NumGroups);
			}
		}

		FRDGBufferSRVRef StepOutputBuffer_SRV = nullptr;

#if WITH_EDITOR
		// DebugPass == 1 - Only sobel
		if (DebugPass == 1)
		{
			StepOutputBuffer_SRV = GraphBuilder.CreateSRV(InitOutputBuffer, BufferFormat);
		}
		else
#endif
		// 2. JumpFlood steps
		{
			FRDGBufferRef StepIntermediateBufferB = nullptr;
			if (bReallocBuffer)
			{
				FRDGBufferDesc StepIntermediateBufferDescB =
					FRDGBufferDesc::CreateBufferDesc(
						sizeof(FVector4f),
						BufferLength);
				StepIntermediateBufferDescB.Usage = EBufferUsageFlags::UnorderedAccess;

				StepIntermediateBufferB = GraphBuilder.CreateBuffer(StepIntermediateBufferDescB, TEXT("GeometryMaskJFStep.IntermediateBufferB"));
				StoredStepIntermediateBufferB = GraphBuilder.ConvertToExternalBuffer(StepIntermediateBufferB);

				GraphBuilder.QueueBufferUpload(StepIntermediateBufferB, BufferData, sizeof(FVector4f) * BufferLength, ERDGInitialDataFlags::NoCopy);
			}
			else
			{
				StepIntermediateBufferB = GraphBuilder.RegisterExternalBuffer(StoredInitOutputBuffer, TEXT("GeometryMaskJFInit.IntermediateBufferB"));
			}

			FRDGBufferUAVRef StepIntermediateBufferA_UAV = GraphBuilder.CreateUAV(InitOutputBuffer, BufferFormat);
			FRDGBufferUAVRef StepIntermediateBufferB_UAV = GraphBuilder.CreateUAV(StepIntermediateBufferB, BufferFormat);

			FRDGBufferSRVRef StepIntermediateBufferA_SRV = GraphBuilder.CreateSRV(InitOutputBuffer, BufferFormat);
			FRDGBufferSRVRef StepIntermediateBufferB_SRV = GraphBuilder.CreateSRV(StepIntermediateBufferB, BufferFormat);

			StepOutputBuffer_SRV = StepIntermediateBufferA_SRV;

			{
				int32 StepCount = UE::GeometryMask::Private::CalculateStepCount(Parameters.Radius);
				int32 MaxStepCount = StepCount;
				int32 BufferIdx = 0;

#if WITH_EDITOR
				if (MaxStepCount > UE::GeometryMask::Private::MaxSteps)
				{
					UE_LOGF(LogGeometryMask, Warning, "JumpFlood requires too many steps (%u), maximum is %u.", MaxStepCount, UE::GeometryMask::Private::MaxSteps);
				}
				
				if (DebugPass > 1)
				{
					MaxStepCount = FMath::Min(MaxStepCount, DebugPass - 1);
				}
#endif

				for (int32 StepIdx = 0; StepIdx < MaxStepCount; ++StepIdx)
				{
					RDG_EVENT_SCOPE_STAT(GraphBuilder, GeometryMaskJFStep, "GeometryMaskJFStep");

					TRACE_CPUPROFILER_EVENT_SCOPE(GeometryMaskJFStep);
					DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FGeometryMaskPostProcess_DistanceField::GeometryMaskJFStep"), STAT_GeometryMask_GeometryMaskJFStep, STATGROUP_GeometryMask);

					FGeometryMaskJFStepCSBase::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeometryMaskJFStepCSBase::FParameters>();
					{
						PassParameters->InputDimensions = InputSize;
						PassParameters->OneOverInputDimensions = OneOverInputSize;
						PassParameters->UVRatioAdjustment = InputHeightOverWidth;

						PassParameters->StepSize = UE::GeometryMask::Private::CalculateStepSize(StepIdx, StepCount);
						PassParameters->InputBuffer = BufferIdx > 0
							? StepIntermediateBufferB_SRV
							: StepIntermediateBufferA_SRV;

						PassParameters->OutputBuffer = BufferIdx > 0
							? StepIntermediateBufferA_UAV
							: StepIntermediateBufferB_UAV;
					}

					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("Step"),
						ERDGPassFlags::Compute,
						StepComputeShader,
						PassParameters,
						NumGroups);

					BufferIdx = (BufferIdx + 1) % 2;
				}

				StepOutputBuffer_SRV = BufferIdx > 0
					? StepIntermediateBufferB_SRV
					: StepIntermediateBufferA_SRV;
			}
		}

		// 3. JF to DF
		{
			{
				RDG_EVENT_SCOPE_STAT(GraphBuilder, GeometryMaskJFtoDF, "GeometryMaskJFtoDF");

				TRACE_CPUPROFILER_EVENT_SCOPE(GeometryMaskJFtoDF);
				DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FGeometryMaskPostProcess_DistanceField::GeometryMaskJFtoDF"), STAT_GeometryMask_GeometryMaskJFtoDF, STATGROUP_GeometryMask);

				FGeometryMaskJFtoDFCSBase::FParameters* PassParameters = GraphBuilder.AllocParameters<FGeometryMaskJFtoDFCSBase::FParameters>();
				{
					const float StepDistanceMultiplier = static_cast<float>(InputSize.GetMax()) / FMath::Max(1, Parameters.Radius);

					PassParameters->InputDimensions = InputSize;
					PassParameters->OneOverInputDimensions = OneOverInputSize;
					PassParameters->UVRatioAdjustment = InputHeightOverWidth;
					PassParameters->StepDistanceMultiplier = StepDistanceMultiplier;
					PassParameters->OriginalInputTexture = InputTexture_SRV;
					PassParameters->OriginalInputSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
					PassParameters->InputBuffer = StepOutputBuffer_SRV;
					PassParameters->OutputTexture = InputTexture_UAV;
				}

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("CopyToDF"),
					ERDGPassFlags::Compute,
					OutputComputeShader,
					PassParameters,
					NumGroups);
			}
		}
	}

	GraphBuilder.Execute();
}
