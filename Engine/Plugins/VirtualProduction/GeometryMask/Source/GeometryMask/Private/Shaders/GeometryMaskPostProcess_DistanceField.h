// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "GeometryMaskConstants.h"
#include "GeometryMaskPostProcess.h"
#include "RenderGraphResources.h"
#include "Templates/SharedPointer.h"

#if WITH_EDITOR
#include "GeometryMaskSettings.h"
#endif

class FRenderTarget;

struct FGeometryMaskPostProcessParameters_DistanceField
{
	bool bCalculateDF = false;
	uint16 SliceIndex = 0;
	double Radius = 16.f;
};

/** */
class FGeometryMaskPostProcess_DistanceField
	: public TGeometryMaskPostProcess<FGeometryMaskPostProcessParameters_DistanceField>
	, public TSharedFromThis<FGeometryMaskPostProcess_DistanceField>
{
	using Super = TGeometryMaskPostProcess<FGeometryMaskPostProcessParameters_DistanceField>;
	
public:
	explicit FGeometryMaskPostProcess_DistanceField(const FGeometryMaskPostProcessParameters_DistanceField& InParameters);

	virtual void Execute(FRenderTarget* InTexture) override;

protected:
	virtual void Execute_RenderThread(FRHICommandListImmediate& InRHICmdList, FRenderTarget* InTexture) override;

private:
	/** Cached to selectively resize resources when needed. */
	TOptional<FIntPoint> LastInputSize;
	TRefCountPtr<FRDGPooledBuffer> StoredInitOutputBuffer;
	TRefCountPtr<FRDGPooledBuffer> StoredStepIntermediateBufferB;
};
