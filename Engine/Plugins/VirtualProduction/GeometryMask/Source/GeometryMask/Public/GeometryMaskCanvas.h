// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IGeometryMaskReadInterface.h"
#include "SceneView.h"
#include "UObject/Object.h"
#include "UObject/WeakInterfacePtr.h"
#include "GeometryMaskCanvas.generated.h"

#define UE_API GEOMETRYMASK_API

class IGeometryMaskWriteInterface;
class UCanvasRenderTarget2D;
class UGeometryMaskCanvasResource;
class UTextureRenderTarget2DArray;
struct FGeometryMaskCanvasSharedData;

/** Called when Writers becomes non-empty. */
using FOnGeometryMaskCanvasActivated = TDelegate<void()>;

/** Called when Writers becomes empty. */
using FOnGeometryMaskCanvasDeactivated = TDelegate<void()>;

/** A uniquely identified Canvas. */
UCLASS(MinimalAPI, BlueprintType, Transient)
class UGeometryMaskCanvas : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif

	UE_API ULevel* GetLevel() const;

	/** Returns all writers. */
    UE_API const TArray<TWeakInterfacePtr<IGeometryMaskWriteInterface>>& GetWriters() const;

	/** Adds a Mask Writer to the canvas. */
	UE_API void AddWriter(const TScriptInterface<IGeometryMaskWriteInterface>& InWriter);

	/** Appends multiple writers to the canvas. */
	UE_API void AddWriters(const TArray<TScriptInterface<IGeometryMaskWriteInterface>>& InWriters);

	/** Remove a writer from this canvas. */
	UE_API void RemoveWriter(const TScriptInterface<IGeometryMaskWriteInterface>& InWriter);

	/** Gets the number of writers for the canvas. */
	UE_API int32 GetNumWriters() const;

	/** Returns true if this is the default/blank canvas. */
	UE_API bool IsDefaultCanvas() const;

	/** Forcibly free the canvas - removed all writers and frees the resource. */
	UE_API void Free();

	UE_DEPRECATED(5.8, "GetTexture (CanvasRenderTarget2D) has been deprecated. Use GetRenderTarget (TextureRenderTarget2DArray) and pass it into Mask_Textures parameter instead")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering", meta=(DeprecatedFunction, DeprecationMessage="GetTexture (CanvasRenderTarget2D) has been deprecated. Use GetRenderTarget (TextureRenderTarget2DArray) and pass it into Mask_Textures parameter instead"))
	UE_API UCanvasRenderTarget2D* GetTexture() const;

	/** Returns the render target 2D array that the canvas draws to */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering")
	UE_API UTextureRenderTarget2DArray* GetRenderTarget() const;

	/** Get the index to the render target slice the canvas draws to */
	UE_API int16 GetRenderTargetSliceIndex() const;

	/** Get the index to the render target slice the canvas draws to */
	UFUNCTION(BlueprintCallable, BlueprintPure, DisplayName="Get Render Target Slice Index", Category = "Rendering")
	UE_API int32 BP_GetRenderTargetSliceIndex() const;

	/** Get the unique canvas id based on the world and canvas name. */
	UE_API const FGeometryMaskCanvasId& GetCanvasId() const;

	/** Returns the canvas name */
	UE_API FName GetCanvasName() const;

	/** Whether blur is applied or not. */
	UE_API bool IsBlurApplied() const;

	/** Sets whether blur is applied or not. */
	UE_API void SetApplyBlur(const bool bInValue);

	/** Gets the current blur strength, if applicable. */
	UE_API double GetBlurStrength() const;

	/** Sets the blur strength. 0.0 will disable blur. */
	UE_API void SetBlurStrength(const double InValue);

	/** Whether feathering is applied or not. */
	UE_API bool IsFeatherApplied() const;

	/** Sets whether feathering is applied or not. */
	UE_API void SetApplyFeather(const bool bInValue);

	/** Gets the current outer feather radius, in pixels. */
	UE_API int32 GetOuterFeatherRadius() const;

	/** Sets the outer feather radius, in pixels. */
	UE_API void SetOuterFeatherRadius(const int32 InValue);

	/** Gets the current inner feather radius, in pixels. */
	UE_API int32 GetInnerFeatherRadius() const;

	/** Sets the inner feather radius, in pixels. */
	UE_API void SetInnerFeatherRadius(const int32 InValue);

	UE_DEPRECATED(5.8, "Color channel has been deprecated in 5.8. Use GetRenderTargetSliceIndex and pass it into the R value in the Mask_TextureIndexVector color parameter. Rest of the values (GBA) should be -1.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering", meta=(DeprecatedFunction, DeprecationMessage="Color channel has been deprecated in 5.8. Use GetRenderTargetSliceIndex and pass it into the R value in the Mask_TextureIndexVector color parameter. Rest of the values (GBA) should be -1"))
	UE_API EGeometryMaskColorChannel GetColorChannel() const;

	/** Called when Writers becomes non-empty. */
	FOnGeometryMaskCanvasActivated& OnActivated()
	{
		return OnActivatedDelegate;
	}

	/** Called when Writers becomes empty. */
	FOnGeometryMaskCanvasDeactivated& OnDeactivated()
	{
		return OnDeactivatedDelegate;
	}

	UE_DEPRECATED(5.8, "Initialize with level and canvas name has been deprecated")
	UE_API void Initialize(const ULevel* InLevel, FName InCanvasName);

	struct FInitParams
	{
		/** Level that this canvas is relevant for */
		const ULevel* Level = nullptr;
		/** Name identifying this canvas within the level */
		FName CanvasName;
		/** Render target to draw to */
		UTextureRenderTarget2DArray* RenderTarget = nullptr;
		/** Index to the slice in the render target to draw to */
		int32 SliceIndex = -1;
		/** Shared data between canvases */
		TSharedPtr<FGeometryMaskCanvasSharedData> SharedData;
	};
	void Initialize(const FInitParams& InInitParams);

	/** Updates the canvas, intended to be called every frame. */
	UE_API void Update(const ULevel* InLevel, FSceneView& InView);

	/** Get the data shared among canvases using the same render target */
	UE_API TSharedPtr<FGeometryMaskCanvasSharedData> GetSharedData() const;

	const UGeometryMaskCanvasResource* GetResource() const
	{
		return CanvasResource;
	}

	UGeometryMaskCanvasResource* GetResourceMutable()
	{
		return CanvasResource;
	}

	UE_DEPRECATED(5.8, "AssignResource has been deprecated as resource management is done by the canvas itself")
	UE_API void AssignResource(UGeometryMaskCanvasResource* InResource, EGeometryMaskColorChannel InColorChannel);

	UE_DEPRECATED(5.8, "FreeResource has been deprecated as resource management is done by the canvas itself")
	UE_API void FreeResource();

public:
	UE_API static FName GetApplyBlurPropertyName();
	UE_API static FName GetBlurStrengthPropertyName();
	UE_API static FName GetApplyFeatherPropertyName();
	UE_API static FName GetOuterFeatherRadiusPropertyName();
	UE_API static FName GetInnerFeatherRadiusPropertyName();

private:
	static const FName ApplyBlurPropertyName;
	static const FName BlurStrengthPropertyName;
	static const FName ApplyFeatherPropertyName;
	static const FName OuterFeatherRadiusPropertyName;
	static const FName InnerFeatherRadiusPropertyName;

private:
	friend class UGeometryMaskCanvasResource;
	
	/** Sorts writers by various criteria for proper rendering order. */
	void SortWriters();
	
	/** Remove all invalid/stale writers. */
	void RemoveInvalidWriters();

	/** Draws all writers to the canvas. */
	void OnDrawToCanvas(const FGeometryMaskDrawingContext& InDrawingContext, FSceneView& InView, FCanvas* InCanvas);

	/** Updates relevant shader parameters for the given color channel. */
	void UpdateRenderParameters();
	
private:
	FOnGeometryMaskCanvasActivated OnActivatedDelegate;
	FOnGeometryMaskCanvasDeactivated OnDeactivatedDelegate;

	UPROPERTY(Transient, DuplicateTransient)
	FGeometryMaskCanvasId CanvasId;

	/** Uniquely identifies this canvas. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Getter, Category = "Canvas", meta = (AllowPrivateAccess = "true"))
	FName CanvasName;

	/** Optional Blur Toggle. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsBlurApplied", Setter = "SetApplyBlur", Category = "Canvas", meta = (AllowPrivateAccess = "true"))
	bool bApplyBlur = false;
	
	/** Optional Blur Strength. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Canvas", meta = (AllowPrivateAccess = "true", EditCondition = "bApplyBlur", EditConditionHides, ClampMin = 0.0))
	double BlurStrength = 16;

	/** Optional Feather Toggle. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "IsFeatherApplied", Setter = "SetApplyFeather", Category = "Canvas", meta = (AllowPrivateAccess = "true"))
	bool bApplyFeather = false;
	
	/** Optional Outer Feather Radius. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Canvas", meta = (AllowPrivateAccess = "true", EditCondition = "bApplyFeather", EditConditionHides, ClampMin = 0))
	int32 OuterFeatherRadius = 16;

	/** Optional Inner Feather Radius. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter, Setter, Category = "Canvas", meta = (AllowPrivateAccess = "true", EditCondition = "bApplyFeather", EditConditionHides, ClampMin = 0))
	int32 InnerFeatherRadius = 16;

	/** Canvas GPU resource to use. */
	UPROPERTY(Transient)
	TObjectPtr<UGeometryMaskCanvasResource> CanvasResource;

	/** List of objects that write to this canvas. */
	TArray<TWeakInterfacePtr<IGeometryMaskWriteInterface>> Writers;
};

#undef UE_API
