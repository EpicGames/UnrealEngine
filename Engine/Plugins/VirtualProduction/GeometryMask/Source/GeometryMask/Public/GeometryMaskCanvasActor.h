// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "IGeometryMaskClient.h"
#include "GeometryMaskCanvasActor.generated.h"

class IGeometryMaskWriteInterface;
class UCanvasRenderTarget2D;
class UGeometryMaskCanvas;
class UTextureRenderTarget2DArray;

/** Wraps a GeometryMaskCanvas, and discovers/registers writers. */
UCLASS(BlueprintType)
class GEOMETRYMASK_API AGeometryMaskCanvasActor : public AActor, public IGeometryMaskClient
{
	GENERATED_BODY()

public:
	AGeometryMaskCanvasActor(const FObjectInitializer& ObjectInitializer);

	/** Identifies the referenced Canvas. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Canvas")
	FName CanvasName;

	UE_DEPRECATED(5.8, "GetTexture (UCanvasRenderTarget2D*) has been deprecated. Use GetRenderTarget (UTextureRenderTarget2DArray*) instead")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering", meta=(DeprecatedFunction, DeprecatedMessage="GetTexture (CanvasRenderTarget2D) has been deprecated in favor of GetRenderTarget (TextureRenderTarget2DArray)"))
	UCanvasRenderTarget2D* GetTexture() const;

	/** Returns the render target 2D array that the canvas draws to */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering")
	UTextureRenderTarget2DArray* GetRenderTarget() const;

	virtual void BeginPlay() override;

	virtual void PostLoad() override;

#if WITH_EDITOR
	virtual void RerunConstructionScripts() override;
#endif

protected:
	/** Resolve/locate the canvas identified by CanvasName. */
	bool TryResolveCanvas();

	/** Find writers on child actors and add them to the referenced canvas. */
	void FindWriters();

	//~ Begin IGeometryMaskClient
	virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	/** Reference to the Canvas used, identified by CanvasName. */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UGeometryMaskCanvas> Canvas;

	/** List of objects that write to this canvas. */
	UPROPERTY()
	TArray<TScriptInterface<IGeometryMaskWriteInterface>> Writers;
};
