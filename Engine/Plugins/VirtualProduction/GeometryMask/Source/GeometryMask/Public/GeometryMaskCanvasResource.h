// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskConstants.h"
#include "GeometryMaskTypes.h"
#include "UObject/Object.h"

#include "GeometryMaskCanvasResource.generated.h"

#define UE_API GEOMETRYMASK_API

class FCanvas;
class FGeometryMaskPostProcess_Blur;
class FGeometryMaskPostProcess_DistanceField;
class FSceneView;
class UCanvasRenderTarget2D;
class UTextureRenderTarget2DArray;
enum class EGeometryMaskColorChannel : uint8;
enum class ETextureRenderTargetSampleCount : uint8;
struct FGeometryMaskCanvasSharedData;

using FOnGeometryMaskCanvasDraw = TMulticastDelegate<void(const FGeometryMaskDrawingContext&, FCanvas*)>;

UCLASS(MinimalAPI)
class UGeometryMaskCanvasResource : public UObject
{
	GENERATED_BODY()

public:
	static constexpr int32 MaxNumChannels = UE::GeometryMask::MaxNumChannels;
	static constexpr int32 MaxNumAvailableChannels = UE::GeometryMask::MaxNumChannels;
	static constexpr int32 MaxTextureSize = UE::GeometryMask::MaxTextureSize;

	UE_API UGeometryMaskCanvasResource();

	UE_API virtual void BeginDestroy() override;

	void Initialize(UTextureRenderTarget2DArray* InRenderTarget, int32 InSliceIndex, const TSharedPtr<FGeometryMaskCanvasSharedData>& InSharedData);

	/** Will return the first available color channel without a canvas assigned. EGeometryMaskColorChannel::None if not available. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API const EGeometryMaskColorChannel GetNextAvailableColorChannel() const;

	/** Requests usage of the given color channel for this resource. Will return true if successful. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API bool Checkout(const EGeometryMaskColorChannel InColorChannel, const FGeometryMaskCanvasId& InRequestingCanvasId);

	/** Returns/frees the color channel associated with the given canvas name. Returns true if canvas name found. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API bool Checkin(const FGeometryMaskCanvasId& InRequestingCanvasId);

	/** Re-arranges used channels such that they are used sequentially. Returns number of unused channels. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API int32 Compact();

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API int32 GetNumChannelsUsed() const;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API bool IsAnyChannelUsed() const;

	/** Get the list of dependent canvases, for debug purposes. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UE_API TArray<FGeometryMaskCanvasId> GetDependentCanvasIds() const;

	/** Gets the data shared between canvases using the same render target*/
	UE_API const TSharedPtr<FGeometryMaskCanvasSharedData>& GetSharedData() const;

	/** 
	 * Validates that the render target size is up-to-date with the viewport size and updating it if not
	 * @return true if the render target size was updated
	 */
	UE_API bool UpdateViewportSize();

	UE_DEPRECATED(5.8, "SetViewportSize deprecated as it's no longer needed")
	UE_API void SetViewportSize(FGeometryMaskDrawingContext& InDrawingContext, const FIntPoint& InViewportSize);

	/** Returns the maximum size that would cover all requested viewports */
	UE_API const FIntPoint& GetMaxViewportSize() const;

	UE_DEPRECATED(5.8, "GetViewportPadding requiring a drawing context has been deprecated. Use GetRequiredViewportPadding() instead")
	UE_API int32 GetViewportPadding(const FGeometryMaskDrawingContext& InDrawingContext) const;

	/** Returns the current viewport padding */
	UE_API int32 GetSharedViewportPadding() const;

	/** Returns the minimum required viewport padding, in pixels - determined by certain effects. */
	UE_API int32 GetRequiredViewportPadding() const;

	UE_DEPRECATED(5.8, "GetRenderTargetTexture (UCanvasRenderTarget2D) has been deprecated in favor of GetRenderTarget (UTextureRenderTarget2DArray)")
	UE_API UCanvasRenderTarget2D* GetRenderTargetTexture();

	/** Returns the render target the resource draws to */
	UE_API UTextureRenderTarget2DArray* GetRenderTarget() const;

	/** Get the index to the render target slice the resource draws to */
	UE_API int16 GetRenderTargetSliceIndex() const;

	UE_DEPRECATED(5.8, "OnDrawToCanvasDelegate has been deprecated as canvas resources are now directly owned by the canvas object")
	UE_API FOnGeometryMaskCanvasDraw& OnDrawToCanvas();

	UE_DEPRECATED(5.8, "Color Channel is no longer used. Use the overload that does not take the color channel")
	UE_API void UpdateRenderParameters(EGeometryMaskColorChannel InColorChannel, bool bInApplyBlur, double InBlurStrength, bool bInApplyFeather, int32 InOuterFeatherRadius, int32 InInnerFeatherRadius);

	UE_API void UpdateRenderParameters(bool bInApplyBlur, double InBlurStrength, bool bInApplyFeather, int32 InOuterFeatherRadius, int32 InInnerFeatherRadius);

	/** Resets the render parameters for the given channel. */
	UE_DEPRECATED(5.8, "Color Channel is no longer used. Use the overload that does not take the color channel")
	UE_API void ResetRenderParameters(EGeometryMaskColorChannel InColorChannel);

	/** Resets the render parameters for the resource */
	UE_API void ResetRenderParameters();

	/** Updates the canvas, intended to be called every frame. */
	UE_API void Update(const ULevel* InLevel, FSceneView& InView, int32 InViewIndex = 0);

private:
	ETextureRenderTargetSampleCount GetSampleCount() const;

	/** Draws all writers to the canvas. */
	void Draw(const ULevel* InLevel, FSceneView& InView, int32 InViewIndex = 0);

	FGeometryMaskDrawingContext* GetDrawingContextForLevel(const ULevel* InLevel, uint8 InSceneViewIndex);
	FGeometryMaskDrawingContext* GetDrawingContextForCanvas(const FGeometryMaskCanvasId& InCanvasId);

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	FGeometryMaskDrawingContext* GetDrawingContextForChannel(EGeometryMaskColorChannel InColorChannel);

	[[maybe_unused]] bool RemoveInvalidDrawingContexts();

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UPROPERTY()
	TMap<EGeometryMaskColorChannel, FGeometryMaskCanvasId> DependentCanvasIds_DEPRECATED;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	TBitArray<TInlineAllocator<UE::GeometryMask::MaxNumChannels>> UsedChannelMask;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UPROPERTY()
	int32 NumUsedChannels_DEPRECATED = 0;

	/** Canvas used for drawing. Using FCanvas over UCanvas for perf */
	TSharedPtr<FCanvas> Canvas;

	/** The default viewport size to use when it can't be resolved from the actual viewport. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Getter = "GetMaxViewportSize", Category = "Rendering", meta = (AllowPrivateAccess = "true"))
	FIntPoint MaxViewportSize = FIntPoint(1920, 1080);

	/** The underlying Render Target texture. */
	UE_DEPRECATED(5.8, "RenderTargetTexture (UCanvasRenderTarget2D) has been deprecated in favor of RenderTarget (UTextureRenderTarget2DArray)")
	UPROPERTY()
	TObjectPtr<UCanvasRenderTarget2D> RenderTargetTexture_DEPRECATED;

	/** The Render Target that this resource uses */
	UPROPERTY(DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2DArray> RenderTarget;

	/** The render target slice assigned to this resource */
	UPROPERTY(DuplicateTransient)
	int16 RenderTargetSliceIndex = -1;

	/** Shared data between canvases using the same render target */
	TSharedPtr<FGeometryMaskCanvasSharedData> SharedData;

	TSet<FGeometryMaskDrawingContext> DrawingContextCache;

	FGeometryMaskDrawingContext DefaultDrawingContext;

	bool bApplyBlur = false;
	bool bApplyDF = false;

	TSharedPtr<FGeometryMaskPostProcess_Blur> PostProcess_Blur;
	TSharedPtr<FGeometryMaskPostProcess_DistanceField> PostProcess_DistanceField;
};

#undef UE_API
