// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/IntPoint.h"

struct FGeometryMaskCanvasSharedData
{
	void Reset()
	{
		ViewportSize = FIntPoint(0, 0);
		TextureSize = FIntPoint(0, 0);
		ViewportPadding = 0;
	}

	/** Viewport size of the canvas */
	FIntPoint ViewportSize = FIntPoint(0, 0);

	/** Actual size of the canvas render target */
	FIntPoint TextureSize = FIntPoint(0, 0);

	/** Viewport padding affecting the texture size */
	int32 ViewportPadding = 0;
};
