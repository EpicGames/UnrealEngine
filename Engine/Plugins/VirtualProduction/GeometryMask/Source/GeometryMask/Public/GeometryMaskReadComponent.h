// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "GeometryMaskTypes.h"
#include "IGeometryMaskReadInterface.h"

#include "GeometryMaskReadComponent.generated.h"

class UCanvasRenderTarget2D;
class UGeometryMaskCanvas;

UCLASS(Blueprintable, BlueprintType, HideCategories=(Activation, Cooking, AssetUserData, Navigation), meta=(BlueprintSpawnableComponent))
class GEOMETRYMASK_API UGeometryMaskReadComponent
	: public UGeometryMaskCanvasReferenceComponentBase
	, public IGeometryMaskReadInterface
{
	GENERATED_BODY()

public:
	//~ Begin IGeometryMaskReadInterface
	virtual void SetParameters(FGeometryMaskReadParameters& InParameters) override;
	virtual const FGeometryMaskReadParameters& GetParameters() const override
	{
		return Parameters;
	}
	virtual FOnGeometryMaskSetCanvasNativeDelegate& OnSetCanvas() override
	{
		return OnSetCanvasDelegate;
	}
	//~ End IGeometryMaskReadInterface

protected:
	//~ Begin IGeometryMaskClient
	virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	//~ Begin UGeometryMaskCanvasReferenceComponentBase
	virtual bool TryResolveCanvas() override;
	//~ End UGeometryMaskCanvasReferenceComponentBase

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter="SetParameters", Getter="GetParameters", Category="Mask", meta=(ShowOnlyInnerProperties))
	FGeometryMaskReadParameters Parameters;
};
