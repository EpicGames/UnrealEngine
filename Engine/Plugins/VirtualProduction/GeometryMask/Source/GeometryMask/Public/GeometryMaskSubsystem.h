// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "GeometryMaskCanvas.h"
#include "GeometryMaskCanvasResource.h"
#include "SceneView.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/WorldSubsystem.h"

#include "GeometryMaskSubsystem.generated.h"

#define UE_API GEOMETRYMASK_API

class UGeometryMaskWorldSubsystem;

using FOnGeometryMaskResourceCreated = TMulticastDelegate<void(const UGeometryMaskCanvasResource*)>;
using FOnGeometryMaskResourceDestroyed = TMulticastDelegate<void(const UGeometryMaskCanvasResource*)>;

/** Maintains the registered named canvases. */
UCLASS(BlueprintType, MinimalAPI)
class UGeometryMaskSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.8, "GetDefaultCanvas is deprecated. Subsystem no longer manages holds a default canvas")
	UFUNCTION(BlueprintCallable, Category = "Canvas", meta=(DeprecatedFunction, DeprecatedMessage="GetDefaultCanvas is deprecated. Subsystem no longer manages holds a default canvas"))
	UE_API UGeometryMaskCanvas* GetDefaultCanvas();

	UE_DEPRECATED(5.8, "CanvasResources is deprecated. Subsystem no longer manages resources")
	UE_API int32 GetNumCanvasResources() const;

	UE_DEPRECATED(5.8, "CanvasResources is deprecated. Subsystem no longer manages resources")
	UE_API const TSet<TObjectPtr<UGeometryMaskCanvasResource>>& GetCanvasResources() const;

	UE_API void Update(UWorld* InWorld, FSceneViewFamily& InViewFamily);

	/** Toggles if no arg given. */
	UE_API void ToggleUpdate(const TOptional<bool>& bInShouldUpdate = {});

	/** Called when a new canvas resource is created. */
	UE_DEPRECATED(5.8, "OnGeometryMaskResourceCreated is deprecated. Subsystem no longer manages resources")
	FOnGeometryMaskResourceCreated& OnGeometryMaskResourceCreated()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OnGeometryMaskResourceCreatedDelegate;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Called when a canvas resource is destroyed. */
	UE_DEPRECATED(5.8, "OnGeometryMaskResourceDestroyed is deprecated. Subsystem no longer manages resources")
	FOnGeometryMaskResourceDestroyed& OnGeometryMaskResourceDestroyed()
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return OnGeometryMaskResourceDestroyedDelegate;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

private:
	UE_API void UpdateLevel(const ULevel* InLevel, UGeometryMaskWorldSubsystem* InWorldSubsystem, FSceneViewFamily& InViewFamily);

	/** Find and assign the next available resource to the given canvas. */
	UE_DEPRECATED(5.8, "AssignResourceToCanvas is deprecated. Subsystem no longer manages resources")
	UE_API void AssignResourceToCanvas(UGeometryMaskCanvas* InCanvas);

	UE_DEPRECATED(5.8, "CompactResources is deprecated. Subsystem no longer manages resources")
	UE_API void CompactResources();

	UE_DEPRECATED(5.8, "OnWorldDestroyed is deprecated. Subsystem no longer manages resources")
	UE_API void OnWorldDestroyed(UWorld* InWorld);

private:
	friend class UGeometryMaskWorldSubsystem;

	UE_DEPRECATED(5.8, "OnGeometryMaskResourceCreatedDelegate is deprecated. Subsystem no longer manages resources")
	FOnGeometryMaskResourceCreated OnGeometryMaskResourceCreatedDelegate;

	UE_DEPRECATED(5.8, "OnGeometryMaskResourceDestroyedDelegate is deprecated. Subsystem no longer manages resources")
	FOnGeometryMaskResourceDestroyed OnGeometryMaskResourceDestroyedDelegate;

	std::atomic<bool> bDoUpdates = true;

	/** Pool of GPU/Texture resources used by the canvases. */
	UE_DEPRECATED(5.8, "CanvasResources is deprecated. Subsystem no longer manages resources")
	UPROPERTY()
	TSet<TObjectPtr<UGeometryMaskCanvasResource>> CanvasResources_DEPRECATED;
};

#undef UE_API
