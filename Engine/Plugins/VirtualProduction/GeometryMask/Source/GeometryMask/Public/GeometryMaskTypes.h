// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "CoreTypes.h"
#include "IGeometryMaskClient.h"
#include "UObject/ObjectKey.h"

#include "GeometryMaskTypes.generated.h"

class UCanvasRenderTarget2D;
class UGeometryMaskCanvas;
class UTextureRenderTarget2DArray;

UENUM(BlueprintType)
enum class EGeometryMaskColorChannel : uint8
{
	Red = 0,
	Green = 1 << 0,
	Blue = 1 << 1,
	Alpha = 1 << 2,
	None = 1 << 3,

	Num
};
ENUM_CLASS_FLAGS(EGeometryMaskColorChannel)

namespace UE::GeometryMask
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Use to convert between a multiplied vector and an EGeometryMaskColorChannel enum value. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	static TMap<FLinearColor, EGeometryMaskColorChannel> VectorToMaskChannelEnum =
	{
		{ {1, 0, 0, 0}, EGeometryMaskColorChannel::Red },
		{ {0, 1, 0, 0}, EGeometryMaskColorChannel::Green },
		{ {0, 0, 1, 0}, EGeometryMaskColorChannel::Blue },
		{ {0, 0, 0, 1}, EGeometryMaskColorChannel::Alpha }
	};

	/** Use to convert between a EGeometryMaskColorChannel enum value and a multiplied vector. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	static TMap<EGeometryMaskColorChannel, FLinearColor> MaskChannelEnumToVector =
	{
		{ EGeometryMaskColorChannel::Red, {1, 0, 0, 0} },
		{ EGeometryMaskColorChannel::Green, {0, 1, 0, 0} },
		{ EGeometryMaskColorChannel::Blue, {0, 0, 1, 0} },
		{ EGeometryMaskColorChannel::Alpha, {0, 0, 0, 1} }
	};

	/** Use to convert between a EGeometryMaskColorChannel enum value and an FColor. */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	static TMap<EGeometryMaskColorChannel, FColor> MaskChannelEnumToColor =
	{
		{ EGeometryMaskColorChannel::Red, {255, 0, 0, 0} },
		{ EGeometryMaskColorChannel::Green, {0, 255, 0, 0} },
		{ EGeometryMaskColorChannel::Blue, {0, 0, 255, 0} },
		{ EGeometryMaskColorChannel::Alpha, {0, 0, 0, 255} }
	};	
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Use to convert between a EGeometryMaskColorChannel and it's value name (as string). */
	static TMap<EGeometryMaskColorChannel, FString> MaskChannelEnumToString =
	{
		{ EGeometryMaskColorChannel::Red, TEXT("Red") },
		{ EGeometryMaskColorChannel::Green, TEXT("Green") },
		{ EGeometryMaskColorChannel::Blue, TEXT("Blue") },
		{ EGeometryMaskColorChannel::Alpha, TEXT("Alpha") },
		{ EGeometryMaskColorChannel::None, TEXT("None") },
		{ EGeometryMaskColorChannel::Num, TEXT("Num/Max") }
	};

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	GEOMETRYMASK_API EGeometryMaskColorChannel VectorToMaskChannel(const FLinearColor& InVector);

	/** Returns a valid ColorChannel (R,G,B or A). */
	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	GEOMETRYMASK_API EGeometryMaskColorChannel GetValidMaskChannel(EGeometryMaskColorChannel InColorChannel, bool bInIncludeAlpha = false);

	GEOMETRYMASK_API FStringView ChannelToString(EGeometryMaskColorChannel InColorChannel);
};

USTRUCT()
struct GEOMETRYMASK_API FGeometryMaskCanvasId
{
	GENERATED_BODY()

public:
	TObjectKey<ULevel> Level;

	UPROPERTY()
	uint8 SceneViewIndex = 0;

	UPROPERTY()
	FName Name;

public:
	static const FGeometryMaskCanvasId& None;
	static const FName DefaultCanvasName;

public:
	FGeometryMaskCanvasId() = default;
	explicit FGeometryMaskCanvasId(const ULevel* InLevel, const FName InName);
	explicit FGeometryMaskCanvasId(EForceInit);
	
	bool IsDefault() const;
	bool IsNone() const;
	void ResetToNone();

	FString ToString() const;

public:
	bool operator==(const FGeometryMaskCanvasId& InOther) const
	{
		return Level == InOther.Level && Name.IsEqual(InOther.Name) && SceneViewIndex == InOther.SceneViewIndex;
	}
	
	bool operator!=(const FGeometryMaskCanvasId& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FGeometryMaskCanvasId& InCanvasId);
};

/** Uniquely identified by world and scene view index. */
USTRUCT()
struct GEOMETRYMASK_API FGeometryMaskDrawingContext
{
	GENERATED_BODY()

public:
	TObjectKey<ULevel> Level;
	
	UPROPERTY()
	uint8 SceneViewIndex = 0;

	UPROPERTY()
	FIntPoint ViewportSize = { 1920, 1080 };

	/** The last resolved ViewProjectionMatrix. */
	UPROPERTY()
	FMatrix ViewProjectionMatrix = FMatrix::Identity;

public:
	FGeometryMaskDrawingContext() = default;
	explicit FGeometryMaskDrawingContext(TObjectKey<ULevel> InLevel, const uint8 InSceneViewIndex = 0);
	explicit FGeometryMaskDrawingContext(const ULevel* InLevel, const uint8 InSceneViewIndex = 0);
	explicit FGeometryMaskDrawingContext(EForceInit);
	
	bool IsValid() const;

public:
	bool operator==(const FGeometryMaskDrawingContext& InOther) const
	{
		return Level == InOther.Level && SceneViewIndex == InOther.SceneViewIndex;
	}

	bool operator!=(const FGeometryMaskDrawingContext& InOther) const
	{
		return !(*this == InOther);
	}

	friend uint32 GetTypeHash(const FGeometryMaskDrawingContext& InUpdateContext);
};


UENUM(BlueprintType)
enum class EGeometryMaskCompositeOperation : uint8
{
	Add,
	Subtract,
	Intersect,

	Num
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnGeometryMaskSetCanvasDelegate, const UGeometryMaskCanvas*, InCanvas);
using FOnGeometryMaskSetCanvasNativeDelegate = TMulticastDelegate<void(const UGeometryMaskCanvas* InCanvas)>;

USTRUCT(BlueprintType)
struct FGeometryMaskReadParameters
{
	GENERATED_BODY()

public:
	/** Specifies the GeometryMaskCanvas to read from. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Canvas")
	FName CanvasName;

	UPROPERTY(EditAnywhere, Category = "Read")
	EGeometryMaskColorChannel ColorChannel = EGeometryMaskColorChannel::Red;

	UPROPERTY(EditAnywhere, Category = "Read")
	bool bInvert = false;
};

USTRUCT(BlueprintType)
struct FGeometryMaskWriteParameters
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGeometryMaskWriteParameters() = default;
	FGeometryMaskWriteParameters(const FGeometryMaskWriteParameters&) = default;
	FGeometryMaskWriteParameters(FGeometryMaskWriteParameters&&) = default;
	FGeometryMaskWriteParameters& operator=(const FGeometryMaskWriteParameters&) = default;
	FGeometryMaskWriteParameters& operator=(FGeometryMaskWriteParameters&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Specifies the GeometryMaskCanvas to reference. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Canvas")
	FName CanvasName;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	UPROPERTY()
	EGeometryMaskColorChannel ColorChannel_DEPRECATED = EGeometryMaskColorChannel::Red;

	UPROPERTY(EditAnywhere, Category = "Write")
	EGeometryMaskCompositeOperation OperationType = EGeometryMaskCompositeOperation::Add;

	UPROPERTY(EditAnywhere, Category = "Write")
	bool bInvert = false;

	/** Higher values draw above lower.  */
	UPROPERTY(EditAnywhere, Category = "Write")
	int32 Priority = 50;

	/** The outer offset of the shapes extent in world units. Smoothly interpolates between the shape/inner radius and this. */
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (ClampMin = 0, UIMin = 0))
	double OuterRadius = 0.0;

	/** The inner offset of the shapes extent in world units. Smoothly interpolates between the shape/outer radius and this. */
	UPROPERTY(EditAnywhere, Category = "Shape", meta = (ClampMin = 0, UIMin = 0))
	double InnerRadius = 0.0;
};

UCLASS(Abstract)
class GEOMETRYMASK_API UGeometryMaskCanvasReferenceComponentBase : public UActorComponent, public IGeometryMaskClient
{
	GENERATED_BODY()

public:
	virtual ~UGeometryMaskCanvasReferenceComponentBase() override;
	
	/** Implement to perform an operation with the provided canvas. */
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, Category = "Canvas", meta = (DisplayName = "Canvas Changed"))
	void ReceiveSetCanvas(const UGeometryMaskCanvas* InCanvas);

	UE_DEPRECATED(5.8, "GetTexture (CanvasRenderTarget2D) has been deprecated. Use GetRenderTarget (TextureRenderTarget2DArray) and pass it into Mask_Textures parameter instead")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering", meta=(DeprecatedFunction, DeprecationMessage="GetTexture (CanvasRenderTarget2D) has been deprecated. Use GetRenderTarget (TextureRenderTarget2DArray) and pass it into Mask_Textures parameter instead"))
	UCanvasRenderTarget2D* GetTexture() const;

	/** Returns the render target 2D array that the canvas draws to */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Rendering")
	UTextureRenderTarget2DArray* GetRenderTarget() const;

protected:
	virtual void BeginPlay() override;
	virtual void PostLoad() override;
	virtual void OnRegister() override;

	/** Override to inject CanvasName into TryResolveCanvas(FName). */
	virtual bool TryResolveCanvas() PURE_VIRTUAL(UGeometryMaskCanvasReferenceComponentBase::TryResolveCanvas, return false; )
	
	bool TryResolveNamedCanvas(FName InCanvasName);

	[[maybe_unused]] virtual bool Cleanup();

	//~ Begin IGeometryMaskClient
	virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	FOnGeometryMaskSetCanvasNativeDelegate OnSetCanvasDelegate;

	/** Reference to the Canvas used, identified by CanvasName. */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)
	TWeakObjectPtr<UGeometryMaskCanvas> CanvasWeak;
};

/** Mesh data for a single mesh, in local space. */
USTRUCT()
struct FGeometryMaskBatchElementData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FVector3f> Vertices;

	UPROPERTY()
	TArray<uint32> Indices;

	UPROPERTY()
	int32 NumTriangles = 0;

	/** Used to check cached state. */
	UPROPERTY()
	uint64 ChangeStamp = 0;

	void Reserve(
		const int32 InNumVertices,
		const int32 InNumIndices,
		const int32 InNumTriangles)
	{
		Vertices.Reserve(InNumVertices);
		Indices.Reserve(InNumIndices);
		NumTriangles = InNumTriangles;
	}
};
