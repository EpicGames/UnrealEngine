// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskSubsystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/SharedPointer.h"
#include "UnrealClient.h"
#include "GeometryMaskWorldSubsystem.generated.h"

#define UE_API GEOMETRYMASK_API

class FGeometryMaskSceneViewExtension;
class IGeometryMaskClient;
class ULevel;
struct FGeometryMaskCanvasSharedData;

using FOnGeometryMaskCanvasCreated = TMulticastDelegate<void(const UGeometryMaskCanvas*)>;
using FOnGeometryMaskCanvasDestroyed = TMulticastDelegate<void(const FGeometryMaskCanvasId&)>;

USTRUCT()
struct FGeometryMaskLevelState
{
	GENERATED_BODY()

	/** All the canvases in the level */
	UPROPERTY()
	TMap<FName, TObjectPtr<UGeometryMaskCanvas>> NamedCanvases;

	/** All the clients using the mask canvases */
	TArray<TWeakInterfacePtr<const IGeometryMaskClient>> MaskClients;

	/** The render target where the canvases in the levels draw to */
	UPROPERTY()
	TObjectPtr<UTextureRenderTarget2DArray> RenderTarget;

	/** Returns the index to the first available slice in the render target */
	int32 AcquireAvailableSliceIndex();

	/** Cleans up all canvases that were registered but are no longer being used by any of the registered mask clients */
	void CleanupUnusedCanvases();
};

/** Updates the canvases. */
UCLASS(MinimalAPI)
class UGeometryMaskWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	UGeometryMaskWorldSubsystem();

	/** Retrieves a Canvas, uniquely identified by this world and the canvas name. */
	UE_API UGeometryMaskCanvas* GetNamedCanvas(const ULevel* InLevel, FName InName);

	/** 
	 * Retrieves a Canvas, uniquely identified by this world and the canvas name.
	 * @param InLevel the level that the canvas relates to
	 * @param InName the name of the canvas
	 * @param InMaskClient the user of the mask (if any)
	 */
	UE_API UGeometryMaskCanvas* GetNamedCanvas(const ULevel* InLevel, FName InName, const IGeometryMaskClient* InMaskClient);

	/** Returns all registered canvas names for this world. */
	UE_API TArray<FName> GetCanvasNames(const ULevel* InLevel);
	
	/** Remove all canvases without any Readers or Writers. Return the number of canvases removed. */
	UE_API int32 RemoveWithoutWriters();
	
	/** Called when a new canvas is created due to a unique name being requested. */
	FOnGeometryMaskCanvasCreated& OnGeometryMaskCanvasCreated()
	{
		return OnGeometryMaskCanvasCreatedDelegate;
	}

	/** Called when a canvas is destroyed due to having no registered writers. */
	FOnGeometryMaskCanvasDestroyed& OnGeometryMaskCanvasDestroyed()
	{
		return OnGeometryMaskCanvasDestroyedDelegate;
	}

protected:
	// ~Begin USubsystem
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	// ~End USubsystem

private:
	UE_API const FGeometryMaskLevelState* FindLevelState(const ULevel* InLevel) const;

	UE_API FGeometryMaskLevelState& FindOrAddLevelState(const ULevel* InLevel);

	friend class UGeometryMaskSubsystem;

	TSharedPtr<FGeometryMaskSceneViewExtension> GeometryMaskSceneViewExtension;

	FOnGeometryMaskCanvasCreated OnGeometryMaskCanvasCreatedDelegate;
	FOnGeometryMaskCanvasDestroyed OnGeometryMaskCanvasDestroyedDelegate;

	UPROPERTY()
	TMap<TWeakObjectPtr<const ULevel>, FGeometryMaskLevelState> LevelStates;

	/** Shared data for canvases of the world presented */
	TSharedPtr<FGeometryMaskCanvasSharedData> SharedData;
};

#undef UE_API
