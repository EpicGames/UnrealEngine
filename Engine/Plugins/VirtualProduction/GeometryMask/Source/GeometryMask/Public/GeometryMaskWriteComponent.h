// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "GeometryMaskTypes.h"
#include "IGeometryMaskWriteInterface.h"
#include "GeometryMaskWriteComponent.generated.h"

#define UE_API GEOMETRYMASK_API

class UCanvasRenderTarget2D;
class UDynamicMeshComponent;
class UPrimitiveComponent;
class UStaticMeshComponent;

namespace UE::GeometryMask
{
	struct FMaskWriter;
}

UCLASS(MinimalAPI, BlueprintType, HideCategories=(Activation, Cooking, AssetUserData, Navigation), meta=(BlueprintSpawnableComponent))
class UGeometryMaskWriteMeshComponent: public UGeometryMaskCanvasReferenceComponentBase, public IGeometryMaskWriteInterface
{
	GENERATED_BODY()

public:
	UE_API UGeometryMaskWriteMeshComponent();

	//~ Begin IGeometryMaskWriteInterface
	UE_API virtual bool IsMaskWriterEnabled() const override;
	UE_API virtual void SetParameters(FGeometryMaskWriteParameters& InParameters) override;
	virtual const FGeometryMaskWriteParameters& GetParameters() const override
	{
		return Parameters;
	}
	UE_API virtual void DrawToCanvas(FCanvas* InCanvas) override;
	virtual FOnGeometryMaskSetCanvasNativeDelegate& OnSetCanvas() override
	{
		return OnSetCanvasDelegate;
	}
	UE_API virtual UE::GeometryMask::FMaskWriter* GetMaskWriter() override;
	//~ End IGeometryMaskWriteInterface

protected:
	//~ Begin UActorComponent
	UE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	//~ End UActorComponent

	//~ Begin IGeometryMaskClient
	UE_API virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const override;
	//~ End IGeometryMaskClient

	//~ Begin UGeometryMaskCanvasReferenceComponentBase
    UE_API virtual bool TryResolveCanvas() override;
	UE_API virtual bool Cleanup() override;
    //~ End UGeometryMaskCanvasReferenceComponentBase

	// Resets cached data, triggers rebuild
	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void ResetCachedData();

	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void UpdateCachedData();

	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void UpdateCachedStaticMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents);

	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void UpdateCachedDynamicMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents);

#if WITH_EDITOR
	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent);
#endif
	UE_DEPRECATED(5.8, "Logic moved to UE::GeometryMask::FMaskWriter")
	UE_API void OnDynamicMeshChanged(UDynamicMeshComponent* InDynamicMeshComponent);
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Mask", meta=(ShowOnlyInnerProperties))
	FGeometryMaskWriteParameters Parameters;

	/** Write to the mask even when this or it's owner is hidden. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Mask")
	bool bWriteWhenHidden = true;

	UE_DEPRECATED(5.8, "Deprecated to use MaskWriter instead")
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecatedMessage="Deprecated to use MaskWriter instead"))
	int32 LastPrimitiveComponentCount;

	UE_DEPRECATED(5.7, "Deprecated to use MaskWriter instead")
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecatedMessage="Deprecated to use MaskWriter instead"))
	TMap<FName, TWeakObjectPtr<UPrimitiveComponent>> CachedComponentsWeak;

	/** Cached primitive component to mesh name, multiple components can use same mesh data */
	UE_DEPRECATED(5.8, "Deprecated to use MaskWriter instead")
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecatedMessage="Deprecated to use MaskWriter instead"))
	TMap<TWeakObjectPtr<UPrimitiveComponent>, FName> CachedComponents;

	/** Cached mesh name to data (one entry per unique mesh) */
	UE_DEPRECATED(5.8, "Deprecated to use MaskWriter instead")
	UPROPERTY(Transient, meta=(DeprecatedProperty, DeprecatedMessage="Deprecated to use MaskWriter instead"))
	TMap<FName, FGeometryMaskBatchElementData> CachedMeshData;

	/** Handles caching and drawing the mask */
	TSharedPtr<UE::GeometryMask::FMaskWriter> MaskWriter;
};

#undef UE_API
