// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GeometryMaskTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectKey.h"

#define UE_API GEOMETRYMASK_API

class UDynamicMeshComponent;
class UPrimitiveComponent;
class UStaticMeshComponent;

namespace UE::GeometryMask
{

/** Handles writing to the mask canvas and caching component draw data */
struct FMaskWriter : public TSharedFromThis<FMaskWriter>
{
	static UE_API TSharedRef<FMaskWriter> Create();

	struct FDrawParams
	{
		/** The canvas to write to */
		FCanvas* Canvas = nullptr;
		/** The actor to gather components from */
		AActor* Actor = nullptr;
		/** Write parameters */
		TNotNull<const FGeometryMaskWriteParameters*> Parameters;
		/** Whether to write the actor if it's hidden */
		bool bWriteWhenHidden = true;
	};
	UE_API void DrawToCanvas(const FDrawParams& InParams);

	/** Iterates each primitive that is masked */
	void ForEachMaskPrimitive(TFunctionRef<void(TNotNull<const UPrimitiveComponent*>)> InFunc) const;

private:
	// Resets cached data, triggers rebuild
	void ResetCachedData();
	void UpdateCachedData(AActor* InActor);
	void UpdateCachedStaticMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents);
	void UpdateCachedDynamicMeshData(TConstArrayView<UPrimitiveComponent*> InPrimitiveComponents);

#if WITH_EDITOR
	void OnStaticMeshChanged(UStaticMeshComponent* InStaticMeshComponent);
#endif
	void OnDynamicMeshChanged(UDynamicMeshComponent* InDynamicMeshComponent);

	/** Cached primitive component to the mesh object used, multiple components can use same mesh data */
	TMap<TObjectKey<UPrimitiveComponent>, FObjectKey> CachedComponents;

	/** Cached mesh object to its batch element data (one entry per unique mesh) */
	TMap<FObjectKey, FGeometryMaskBatchElementData> CachedMeshData;

	/** Cached component count per actor */
	TMap<TObjectKey<AActor>, int32> LastPrimitiveComponentCountMap;
};

} // UE::GeometryMask

#undef UE_API
