// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "IGeometryMaskClient.generated.h"

class UGeometryMaskCanvas;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UGeometryMaskClient : public UInterface
{
	GENERATED_BODY()
};

/** Interface for objects using canvases */
class IGeometryMaskClient
{
	GENERATED_BODY()

public:
	/** Iterates each canvas that the client uses */
	virtual bool ForEachUsedCanvasName(TFunctionRef<bool(FName)> InFunc) const = 0;
};
