// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskTypes.h"
#include "UObject/Interface.h"
#include "IGeometryMaskReadInterface.generated.h"

class FCanvas;
struct FGeometryMaskReadParameters;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UGeometryMaskReadInterface : public UInterface
{
	GENERATED_BODY()
};

/** Implement to Read from a host GeometryMaskCanvas. */
class IGeometryMaskReadInterface
{
	GENERATED_BODY()

public:
	virtual const FGeometryMaskReadParameters& GetParameters() const = 0;
	virtual void SetParameters(FGeometryMaskReadParameters& InParameters) = 0;
	virtual FOnGeometryMaskSetCanvasNativeDelegate& OnSetCanvas() = 0;
};
