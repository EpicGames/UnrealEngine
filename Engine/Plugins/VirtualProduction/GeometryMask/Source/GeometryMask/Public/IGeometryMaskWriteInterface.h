// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskTypes.h"
#include "UObject/Interface.h"
#include "IGeometryMaskWriteInterface.generated.h"

namespace UE::GeometryMask
{
	struct FMaskWriter;
}

class FCanvas;
struct FGeometryMaskWriteParameters;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UGeometryMaskWriteInterface : public UInterface
{
	GENERATED_BODY()
};

/** Implement to write to a host canvas. */
class IGeometryMaskWriteInterface
{
	GENERATED_BODY()

public:
	virtual bool IsMaskWriterEnabled() const
	{
		return true;
	}
	virtual const FGeometryMaskWriteParameters& GetParameters() const = 0;
	virtual void SetParameters(FGeometryMaskWriteParameters& InParameters) = 0;
	virtual void DrawToCanvas(FCanvas* InCanvas) = 0;
	virtual FOnGeometryMaskSetCanvasNativeDelegate& OnSetCanvas() = 0;

	/** Gets the mask writer for this interface */
	virtual UE::GeometryMask::FMaskWriter* GetMaskWriter()
	{
		ensureMsgf(false, TEXT("5.8: Geometry Mask Write Interface does not implement 'GetMaskWriter'!"));
		return nullptr;
	}
};
