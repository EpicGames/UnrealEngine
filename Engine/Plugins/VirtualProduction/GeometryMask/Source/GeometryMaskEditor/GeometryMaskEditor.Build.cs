// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryMaskEditor : ModuleRules
{
	public GeometryMaskEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
                "GeometryMask",
                "SlateCore",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Engine",
				"InputCore",
				"Slate",
				"UnrealEd",
				"WorkspaceMenuStructure",
			});
    }
}
