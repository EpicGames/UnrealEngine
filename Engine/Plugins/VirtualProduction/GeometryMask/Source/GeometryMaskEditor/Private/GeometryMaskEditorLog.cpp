// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskEditorLog.h"

DEFINE_LOG_CATEGORY(LogGeometryMaskEditor);
