// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryMaskEditorModule.h"

#include "Engine/Engine.h"
#include "Framework/Docking/TabManager.h"
#include "GeometryMaskEditorLog.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "Materials/Material.h"
#include "Styling/SlateIconFinder.h"
#include "ViewModels/GMECanvasListViewModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SGMECanvasList.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FGeometryMaskEditorModule"

const FName FGeometryMaskEditorModule::VisualizerTabId = TEXT("GeometryMaskVisualizer");

void FGeometryMaskEditorModule::StartupModule()
{
	const FText TabText = LOCTEXT("GeometryMaskVisualizerTabName", "Geometry Mask");
	TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();

	GlobalTabManager->RegisterNomadTabSpawner(
			VisualizerTabId,
			FOnSpawnTab::CreateLambda(
				[this](const FSpawnTabArgs&)
				{
					TSharedRef<FGMECanvasListViewModel> CanvasListViewModel = FGMECanvasListViewModel::Create();
					TSharedRef<SDockTab> DockTab =
						SNew(SDockTab)
						.TabRole(ETabRole::NomadTab)
						.Content()
						[
							SNew(SGMECanvasList, CanvasListViewModel)
						];

					return DockTab;
				}
			)
		)
		.SetDisplayNameAttribute(TabText)
		.SetDisplayName(TabText)
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
		.SetIcon(FSlateIconFinder::FindIconForClass(UMaterial::StaticClass()))
		.SetAutoGenerateMenuEntry(false); // Only show via console command
	
	if (GIsEditor && !IsRunningCommandlet())
	{
		RegisterConsoleCommands();
	}
}

void FGeometryMaskEditorModule::ShutdownModule()
{
	const TSharedRef<FGlobalTabmanager> GlobalTabManager = FGlobalTabmanager::Get();
	GlobalTabManager->UnregisterNomadTabSpawner(VisualizerTabId);
	VisualizerTabWeak.Reset();
	
	// Cleanup commands
	UnregisterConsoleCommands();
}

void FGeometryMaskEditorModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("GeometryMask.Visualize"),
		TEXT("Displays a window listing all active GeometryMaskCanvas objects"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FGeometryMaskEditorModule::ExecuteShowVisualizer),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("GeometryMask.Pause"),
		TEXT("Disable ticking of GeometryMask objects"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FGeometryMaskEditorModule::ExecutePause),
		ECVF_Default
	));

	ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("GeometryMask.Flush"),
		TEXT("Flush unused canvas's"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FGeometryMaskEditorModule::ExecuteFlush),
		ECVF_Default
	));
}

void FGeometryMaskEditorModule::UnregisterConsoleCommands()
{
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}

	ConsoleCommands.Empty();
}

void FGeometryMaskEditorModule::ExecuteShowVisualizer(const TArray<FString>& InArgs)
{
	if (!VisualizerTabWeak.IsValid())
	{
		VisualizerTabWeak = FGlobalTabmanager::Get()->TryInvokeTab(VisualizerTabId); 
	}
}

void FGeometryMaskEditorModule::ExecutePause(const TArray<FString>& InArgs)
{
	if (UGeometryMaskSubsystem* GeometryMaskSubsystem = GEngine->GetEngineSubsystem<UGeometryMaskSubsystem>())
	{
		TOptional<bool> bShouldPauseUpdate;
		if (!InArgs.IsEmpty())
		{
			FString ArgStr = InArgs[0];
			if (ArgStr.IsNumeric())
			{
				// Be explicit if arg given
				GeometryMaskSubsystem->ToggleUpdate(FCString::Atoi64(*ArgStr) == 0);
				return;
			}
		}

		// Otherwise toggle
		GeometryMaskSubsystem->ToggleUpdate();
	}
}

void FGeometryMaskEditorModule::ExecuteFlush(const TArray<FString>& InArgs)
{
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (const UWorld* World = WorldContext.World())
		{
			if (UGeometryMaskWorldSubsystem* GeometryMaskSubsystem = World->GetSubsystem<UGeometryMaskWorldSubsystem>())
			{
				const int32 RemovedCanvasNum = GeometryMaskSubsystem->RemoveWithoutWriters();

				int32 ActiveCanvasNum = 0;
				for (const ULevel* Level : World->GetLevels())
				{
					ActiveCanvasNum += GeometryMaskSubsystem->GetCanvasNames(Level).Num();
				}
				UE_LOGF(LogGeometryMaskEditor, Display, "%u canvas's removed because they had no writers - %u canvas's remaining.", RemovedCanvasNum, ActiveCanvasNum);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FGeometryMaskEditorModule, GeometryMaskEditor)
