// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IConsoleObject;
class SDockTab;

class FGeometryMaskEditorModule
	: public IModuleInterface
{
public:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

private:
	/** Called from StartupModule and sets up console commands for the plugin via IConsoleManager */
	void RegisterConsoleCommands();
	
	/** Called from ShutdownModule and clears out previously registered console commands */
	void UnregisterConsoleCommands();

	void ExecuteShowVisualizer(const TArray<FString>& InArgs);

	void ExecutePause(const TArray<FString>& InArgs);

	void ExecuteFlush(const TArray<FString>& InArgs);

private:
	static const FName VisualizerTabId;
	
	/** References of registered console commands via IConsoleManager */
	TArray<IConsoleObject*> ConsoleCommands;

	TWeakPtr<SDockTab> VisualizerTabWeak;
};
