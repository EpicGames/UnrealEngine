// Copyright Epic Games, Inc. All Rights Reserved.

#include "GMECanvasItemViewModel.h"

#include "Engine/Level.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "GeometryMaskCanvas.h"

TSharedRef<FGMECanvasItemViewModel> FGMECanvasItemViewModel::Create(const TWeakObjectPtr<const UGeometryMaskCanvas>& InCanvas)
{
	TSharedRef<FGMECanvasItemViewModel> ViewModel = MakeShared<FGMECanvasItemViewModel>(FPrivateToken{}, InCanvas);

	return ViewModel;
}

FGMECanvasItemViewModel::FGMECanvasItemViewModel(
	FPrivateToken,
	const TWeakObjectPtr<const UGeometryMaskCanvas>& InCanvas)
	: CanvasWeak(InCanvas)
	, KnownReaderCount(0)
	, KnownWriterCount(0)
{
	if (const UGeometryMaskCanvas* Canvas = InCanvas.Get())
	{
		CanvasId = Canvas->GetCanvasId();		
		CanvasTextureWeak = Canvas->GetRenderTarget();
		KnownWriterCount = Canvas->GetWriters().Num();

		UpdateInfoText();
	}
}

const UTexture* FGMECanvasItemViewModel::GetCanvasTexture() const
{
	if (CanvasTextureWeak.IsValid())
	{
		return CanvasTextureWeak.Get();
	}
	
	return nullptr;
}

float FGMECanvasItemViewModel::GetMemoryUsage()
{
	if (const UTexture* Texture = GetCanvasTexture())
	{
		return (static_cast<float>(Texture->CalcTextureMemorySizeEnum(TMC_ResidentMips)) / (1024.0f * 1024.0f));
	}
	
	return 0.0f;
}

void FGMECanvasItemViewModel::UpdateInfoText()
{
	if (const UGeometryMaskCanvas* Canvas = CanvasWeak.Get())
	{
		FGeometryMaskCanvasId Id = Canvas->GetCanvasId();

		FString WorldTypeLabel = TEXT("");
		FString WorldLabel = TEXT("(None)");
		FString LevelLabel = TEXT("(None)");
		if (ULevel* CanvasLevel = Id.Level.ResolveObjectPtr())
		{
			if (CanvasLevel->OwningWorld)
			{
				WorldLabel = CanvasLevel->OwningWorld->GetName();
				WorldTypeLabel = LexToString(CanvasLevel->OwningWorld->WorldType);
			}
			LevelLabel = CanvasLevel->GetName();
		}

		static const FString DefaultCanvasLabel = FGeometryMaskCanvasId::DefaultCanvasName.ToString();
		FString CanvasLabel = DefaultCanvasLabel;
		if (!Id.IsDefault())
		{
			CanvasLabel = Id.Name.ToString();
		}

		InfoText = FText::FromString(
			FString::Printf(TEXT("%-s\n%-12s: %s\n%-12s:  %s\n%-12s: %s\n%-12s: %s\n%-12s: %u"),
				*FString::Printf(TEXT("%s.%s"), *LevelLabel, *CanvasLabel),
				TEXT("World Type"),	*WorldTypeLabel,
				TEXT("World"), *WorldLabel,
				TEXT("Level"), *LevelLabel,
				TEXT("Name"), *CanvasLabel,
				TEXT("Num. Writers"), Canvas->GetNumWriters()));
	}
}

bool FGMECanvasItemViewModel::Tick(const float InDeltaSeconds)
{
	UpdateInfoText();
	return true;
}

bool FGMECanvasItemViewModel::GetChildren(TArray<TSharedPtr<IGMETreeNodeViewModel>>& OutChildren)
{
	return false;
}

const FGeometryMaskCanvasId& FGMECanvasItemViewModel::GetCanvasId() const
{
	return CanvasId;
}

FName FGMECanvasItemViewModel::GetCanvasName() const
{
	return CanvasId.Name;
}

const FText& FGMECanvasItemViewModel::GetCanvasInfo() const
{
	return InfoText;
}
