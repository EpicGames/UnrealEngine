// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "EditorUndoClient.h"
#include "GeometryMaskCanvasResource.h"
#include "GeometryMaskTypes.h"
#include "GMEViewModelShared.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class UGeometryMaskCanvas;
class UTexture;

class FGMECanvasItemViewModel
	: public FGMETickableViewModelBase
	, public FEditorUndoClient
	, public IGMETreeNodeViewModel
{
protected:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	static TSharedRef<FGMECanvasItemViewModel> Create(const TWeakObjectPtr<const UGeometryMaskCanvas>& InCanvas);

	FGMECanvasItemViewModel(FPrivateToken, const TWeakObjectPtr<const UGeometryMaskCanvas>& InCanvas);
	virtual ~FGMECanvasItemViewModel() override = default;

	virtual bool Tick(const float InDeltaSeconds) override;
	
	// ~Begin IGMETreeNodeViewModel
	virtual bool GetChildren(TArray<TSharedPtr<IGMETreeNodeViewModel>>& OutChildren) override;
	// ~End IGMETreeNodeViewModel

	const FGeometryMaskCanvasId& GetCanvasId() const;
	FName GetCanvasName() const;
	const FText& GetCanvasInfo() const;
	const UTexture* GetCanvasTexture() const;
	float GetMemoryUsage();

private:
	void UpdateInfoText();

private:
	FGeometryMaskCanvasId CanvasId;
	FText InfoText;
	TWeakObjectPtr<const UGeometryMaskCanvas> CanvasWeak;
	TWeakObjectPtr<UTexture> CanvasTextureWeak;
	int32 KnownReaderCount;
	int32 KnownWriterCount;
};
