// Copyright Epic Games, Inc. All Rights Reserved.

#include "GMECanvasListViewModel.h"

#include "Editor.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskWorldSubsystem.h"
#include "GMECanvasItemViewModel.h"

TSharedRef<FGMECanvasListViewModel> FGMECanvasListViewModel::Create()
{
	TSharedRef<FGMECanvasListViewModel> ViewModel = MakeShared<FGMECanvasListViewModel>(FPrivateToken{});
	ViewModel->Initialize();

	return ViewModel;
}

FGMECanvasListViewModel::~FGMECanvasListViewModel()
{
	for (const TObjectKey<UWorld>& WorldKey : LoadedWorlds)
	{
		if (UWorld* World = WorldKey.ResolveObjectPtr())
		{
			if (UGeometryMaskWorldSubsystem* Subsystem = World->GetSubsystem<UGeometryMaskWorldSubsystem>())
			{
				for (ULevel* Level : World->GetLevels())
				{
					if (FDelegateHandle* DelegateHandle = OnCanvasCreatedHandles.Find(Level))
					{
						Subsystem->OnGeometryMaskCanvasCreated().Remove(*DelegateHandle);
						DelegateHandle->Reset();
					}

					if (FDelegateHandle* DelegateHandle = OnCanvasDestroyedHandles.Find(Level))
					{
						Subsystem->OnGeometryMaskCanvasDestroyed().Remove(*DelegateHandle);
						DelegateHandle->Reset();
					}
				}
			}
		}
	}

	OnCanvasCreatedHandles.Reset();
	OnCanvasDestroyedHandles.Reset();
	
	CanvasItems.Reset();
}

bool FGMECanvasListViewModel::RefreshItems()
{
	int32 NumCurrentCanvasItems = CanvasItems.Num();

	// Add currently registered canvases
	{
		CanvasItems.Reset();

		for (const TObjectKey<UWorld>& WorldKey : LoadedWorlds)
		{
			UWorld* World = WorldKey.ResolveObjectPtr();
			if (!World)
			{
				continue;
			}

			UGeometryMaskWorldSubsystem* Subsystem = World->GetSubsystem<UGeometryMaskWorldSubsystem>();
			if (!Subsystem)
			{
				continue;
			}

			for (const ULevel* Level : World->GetLevels())
			{
				TArray<FName> CanvasNames = Subsystem->GetCanvasNames(Level);
				for (const FName CanvasName : CanvasNames)
				{
					if (UGeometryMaskCanvas* Canvas = Subsystem->GetNamedCanvas(Level, CanvasName))
					{
						CanvasItems.Add(FGMECanvasItemViewModel::Create(Canvas));
					}
				}

				OnCanvasCreatedHandles.Emplace(Level, Subsystem->OnGeometryMaskCanvasCreated().AddSP(this, &FGMECanvasListViewModel::OnCanvasCreated));
				OnCanvasDestroyedHandles.Emplace(Level, Subsystem->OnGeometryMaskCanvasDestroyed().AddSP(this, &FGMECanvasListViewModel::OnCanvasDestroyed));
			}
		}
	}

	return NumCurrentCanvasItems != CanvasItems.Num();
}

void FGMECanvasListViewModel::OnPostWorldInit(UWorld* InWorld, const UWorld::InitializationValues InWorldValues)
{
	FGMEListViewModelBase::OnPostWorldInit(InWorld, InWorldValues);

	// Listen for new canvases, and destroyed ones
	{
		for (const TObjectKey<UWorld>& WorldKey : LoadedWorlds)
		{
			if (UWorld* World = WorldKey.ResolveObjectPtr())
			{
				if (UGeometryMaskWorldSubsystem* Subsystem = World->GetSubsystem<UGeometryMaskWorldSubsystem>())
				{
					for (const ULevel* Level : World->GetLevels())
					{
						OnCanvasCreatedHandles.Emplace(Level, Subsystem->OnGeometryMaskCanvasCreated().AddSP(this, &FGMECanvasListViewModel::OnCanvasCreated));
						OnCanvasDestroyedHandles.Emplace(Level, Subsystem->OnGeometryMaskCanvasDestroyed().AddSP(this, &FGMECanvasListViewModel::OnCanvasDestroyed));
					}
				}
			}
		}
	}
}

void FGMECanvasListViewModel::OnPreWorldDestroyed(UWorld* InWorld)
{
	FGMEListViewModelBase::OnPreWorldDestroyed(InWorld);

	for (const TObjectKey<UWorld>& WorldKey : LoadedWorlds)
	{
		if (UWorld* World = WorldKey.ResolveObjectPtr())
		{
			if (UGeometryMaskWorldSubsystem* Subsystem = World->GetSubsystem<UGeometryMaskWorldSubsystem>())
			{
				for (const ULevel* Level : World->GetLevels())
				{
					if (FDelegateHandle* DelegateHandle = OnCanvasCreatedHandles.Find(Level))
					{
						Subsystem->OnGeometryMaskCanvasCreated().Remove(*DelegateHandle);
					}

					if (FDelegateHandle* DelegateHandle = OnCanvasDestroyedHandles.Find(Level))
					{
						Subsystem->OnGeometryMaskCanvasDestroyed().Remove(*DelegateHandle);
					}
				}
			}
		}
	}
}

void FGMECanvasListViewModel::OnCanvasCreated(const UGeometryMaskCanvas* InGeometryMaskCanvas)
{
	// Don't add if already in list
	if (CanvasItems.ContainsByPredicate([CanvasName = InGeometryMaskCanvas->GetCanvasName()](const TSharedPtr<FGMECanvasItemViewModel>& InItemViewModel)
	{
		return InItemViewModel->GetCanvasName() == CanvasName;
	}))
	{
		return;
	}

	CanvasItems.Add(FGMECanvasItemViewModel::Create(InGeometryMaskCanvas));

	OnChanged().Broadcast();
}

void FGMECanvasListViewModel::OnCanvasDestroyed(const FGeometryMaskCanvasId& InGeometryMaskCanvasId)
{
	CanvasItems.RemoveAll([&InGeometryMaskCanvasId](const TSharedPtr<FGMECanvasItemViewModel>& InItem)
	{
		return InItem->GetCanvasId() == InGeometryMaskCanvasId;
	});
	
	OnChanged().Broadcast();
}

bool FGMECanvasListViewModel::GetChildren(TArray<TSharedPtr<IGMETreeNodeViewModel>>& OutChildren)
{
	const TArray<TSharedPtr<FGMECanvasItemViewModel>> Children = CanvasItems;
	OutChildren.Append(Children);
	return Children.Num() > 0;
}
