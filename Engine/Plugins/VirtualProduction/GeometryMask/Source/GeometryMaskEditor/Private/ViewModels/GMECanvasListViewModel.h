// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "EditorUndoClient.h"
#include "GeometryMaskCanvas.h"
#include "GMEViewModelShared.h"

class FGMECanvasItemViewModel;

class FGMECanvasListViewModel
	: public FGMEListViewModelBase
	, public FEditorUndoClient
	, public IGMETreeNodeViewModel
{
	using Super = FGMEListViewModelBase;

protected:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	static TSharedRef<FGMECanvasListViewModel> Create();

	explicit FGMECanvasListViewModel(FPrivateToken)
		: Super(Super::FPrivateToken{}) { }
	virtual ~FGMECanvasListViewModel() override;

	// ~Begin IGMETreeNodeViewModel
	virtual bool GetChildren(TArray<TSharedPtr<IGMETreeNodeViewModel>>& OutChildren) override;
	// ~End IGMETreeNodeViewModel

private:
	virtual bool RefreshItems() override;

	virtual void OnPostWorldInit(UWorld* InWorld, const UWorld::InitializationValues InWorldValues) override;
	virtual void OnPreWorldDestroyed(UWorld* InWorld) override;

	void OnCanvasCreated(const UGeometryMaskCanvas* InGeometryMaskCanvas);
	void OnCanvasDestroyed(const FGeometryMaskCanvasId& InGeometryMaskCanvasId);

private:
	TMap<TObjectKey<ULevel>, FDelegateHandle> OnCanvasCreatedHandles;
	TMap<TObjectKey<ULevel>, FDelegateHandle> OnCanvasDestroyedHandles;

	/** Cached canvas names for comparison/refresh. */
	TMap<TObjectKey<ULevel>, TArray<FName>> LastCanvasNames;

	/** Canvas ViewModels */
	TArray<TSharedPtr<FGMECanvasItemViewModel>> CanvasItems;
};
