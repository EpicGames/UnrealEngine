// Copyright Epic Games, Inc. All Rights Reserved.

#include "GMEViewModelShared.h"

#include "Engine/Engine.h"

FGMETickableViewModelBase::FGMETickableViewModelBase()
{
	UpdateCheckHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FGMETickableViewModelBase::Tick), UpdateCheckInterval);
}

FGMETickableViewModelBase::~FGMETickableViewModelBase()
{
	FTSTicker::RemoveTicker(UpdateCheckHandle);
}

FGMEListViewModelBase::FGMEListViewModelBase(FPrivateToken)
{
	OnPostWorldInitDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddRaw(this, &FGMEListViewModelBase::OnPostWorldInit);
	OnPreWorldDestroyedDelegateHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddRaw(this, &FGMEListViewModelBase::OnPreWorldDestroyed);
}

FGMEListViewModelBase::~FGMEListViewModelBase()
{
	FWorldDelegates::OnPostWorldInitialization.Remove(OnPostWorldInitDelegateHandle);
	FWorldDelegates::OnPreWorldFinishDestroy.Remove(OnPreWorldDestroyedDelegateHandle);
}

void FGMEListViewModelBase::Initialize()
{
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (UWorld* World = WorldContext.World())
		{
			LoadedWorlds.Emplace(World);
		}
	}
	
	RefreshItems();
}

bool FGMEListViewModelBase::Tick(const float InDeltaSeconds)
{
	if (RefreshItems())
	{
		OnChanged().Broadcast();
	}

	return true;
}

void FGMEListViewModelBase::OnPostWorldInit(UWorld* InWorld, const UWorld::InitializationValues InWorldValues)
{
	if (LoadedWorlds.AddUnique(InWorld) == LoadedWorlds.Num() - 1)
	{
		RefreshItems();
	}
}

void FGMEListViewModelBase::OnPreWorldDestroyed(UWorld* InWorld)
{
	if (LoadedWorlds.Remove(InWorld) > 0)
	{
		RefreshItems();
	}
}
