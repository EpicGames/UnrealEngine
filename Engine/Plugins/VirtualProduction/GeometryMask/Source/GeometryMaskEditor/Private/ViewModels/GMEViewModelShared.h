// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Ticker.h"
#include "Engine/World.h"
#include "Templates/SharedPointer.h"


/** Common interface for treeview nodes. */
class IGMETreeNodeViewModel
{
protected:
	~IGMETreeNodeViewModel() = default;

public:
	/** See STreeView::OnGetChildren, return false if no children. */
	virtual bool GetChildren(TArray<TSharedPtr<IGMETreeNodeViewModel>>& OutChildren) = 0;
};

class FGMETickableViewModelBase
	: public TSharedFromThis<FGMETickableViewModelBase>
{
public:
	FGMETickableViewModelBase();
	virtual ~FGMETickableViewModelBase();

protected:
	virtual bool Tick(const float InDeltaSeconds) { return true; }

protected:
	/** In seconds. */
	static constexpr float UpdateCheckInterval = 0.15f;
	FTSTicker::FDelegateHandle UpdateCheckHandle;
};

class FGMEListViewModelBase
	: public FGMETickableViewModelBase
{
protected:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	explicit FGMEListViewModelBase(FPrivateToken);
	virtual ~FGMEListViewModelBase() override;

public:
	using FOnChanged = TMulticastDelegate<void()>;;

	/** Something has changed within the ViewModel */
	FOnChanged& OnChanged() { return OnChangedDelegate; }

protected:
	virtual void Initialize();
	virtual bool RefreshItems() = 0;

	virtual bool Tick(const float InDeltaSeconds) override;

	virtual void OnPostWorldInit(UWorld* InWorld, const UWorld::InitializationValues InWorldValues);
	virtual void OnPreWorldDestroyed(UWorld* InWorld);

protected:
	/** In seconds. */
	static constexpr float UpdateCheckInterval = 1.0f;
	FTSTicker::FDelegateHandle UpdateCheckHandle;
	
	TArray<TObjectKey<UWorld>> LoadedWorlds;

	FDelegateHandle OnPostWorldInitDelegateHandle;
	FDelegateHandle OnPreWorldDestroyedDelegateHandle;	
	
	FOnChanged OnChangedDelegate;
};
