// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGMECanvasItem.h"

#include "Engine/Texture.h"
#include "SlateOptMacros.h"
#include "ViewModels/GMECanvasItemViewModel.h"
#include "Widgets/SGeometryMaskCanvasPreview.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

void SGMECanvasItem::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& InOwnerTableView,
	const TSharedRef<FGMECanvasItemViewModel>& InViewModel)
{
	ViewModel = InViewModel;

	ImageWidget = SNew(SGeometryMaskCanvasPreview)
					.Invert(true)
					.SolidBackground(true)
					.CanvasId(ViewModel->GetCanvasId())
					.Opacity(1.0f);

	SGMEImageItem::Construct(
		SGMEImageItem::FArguments()
		.Label(this, &SGMECanvasItem::GetLabel)
		, InOwnerTableView);
}

FText SGMECanvasItem::GetLabel() const
{
	if (ViewModel.IsValid())
	{
		return ViewModel->GetCanvasInfo();
	}

	return FText::GetEmpty();
}

FOptionalSize SGMECanvasItem::GetAspectRatio()
{
	float AspectRatio = 16.0f / 9.0f;
	if (const UTexture* CanvasTexture = ViewModel->GetCanvasTexture())
	{
		AspectRatio = CanvasTexture->GetSurfaceWidth() / CanvasTexture->GetSurfaceHeight();
	}

	return AspectRatio;
}
