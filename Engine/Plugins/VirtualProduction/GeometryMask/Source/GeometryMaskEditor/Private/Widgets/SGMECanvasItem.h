// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGMEImageItem.h"
#include "Templates/SharedPointer.h"

class FGMECanvasItemViewModel;
class IGMETreeNodeViewModel;
class ITableRow;
class STableViewBase;

class SGMECanvasItem
	: public SGMEImageItem<FGMECanvasItemViewModel>
{
public:
	SLATE_BEGIN_ARGS(SGMECanvasItem) {}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FGMECanvasItemViewModel>& InViewModel);

protected:
	FText GetLabel() const;
	virtual FOptionalSize GetAspectRatio() override;

private:
	TSharedPtr<FGMECanvasItemViewModel> ViewModel;
};
