// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGMECanvasList.h"

#include "SGMECanvasItem.h"
#include "SlateOptMacros.h"
#include "ViewModels/GMECanvasItemViewModel.h"
#include "ViewModels/GMECanvasListViewModel.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SGMECanvasList"

SGMECanvasList::~SGMECanvasList()
{
	if (ViewModel.IsValid())
	{
		ViewModel->OnChanged().RemoveAll(this);
	}
}

void SGMECanvasList::Construct(const FArguments& InArgs, TSharedRef<FGMECanvasListViewModel> InViewModel)
{
	constexpr float Padding = 2.0f;
	
	ViewModel = InViewModel;
	ViewModel->OnChanged().AddLambda(
		[this]()
		{
			Refresh();
		});

	SAssignNew(CanvasListWidget, SListView<TSharedPtr<IGMETreeNodeViewModel>>)
		.ListItemsSource(&CanvasItems)
		.OnGenerateRow(this, &SGMECanvasList::OnGenerateRow)
		.ScrollbarVisibility(EVisibility::Visible)
		.SelectionMode(ESelectionMode::None);

	Refresh();

	TWeakPtr<SGMECanvasList> ThisWeak = SharedThis(this);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(Padding)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TotalCount", "Total:"))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(Padding)
				[
					SNew(STextBlock)
					.Text_Lambda([ThisWeak]()
					{
						if (ThisWeak.IsValid())
						{
							return FText::AsNumber(ThisWeak.Pin()->CanvasItems.Num());
						}
						return FText::AsNumber(0);
					})
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(1, 1, 1, Padding)
		[
			CanvasListWidget.ToSharedRef()
		]
	];
}

TSharedRef<ITableRow> SGMECanvasList::OnGenerateRow(
	TSharedPtr<IGMETreeNodeViewModel> InViewModel
	,const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InViewModel.IsValid());

	return SNew(SGMECanvasItem, InOwnerTable, StaticCastSharedPtr<FGMECanvasItemViewModel>(InViewModel).ToSharedRef());
}

void SGMECanvasList::Refresh()
{
	CanvasItems.Reset();
	ViewModel->GetChildren(CanvasItems);
	
	CanvasListWidget->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
