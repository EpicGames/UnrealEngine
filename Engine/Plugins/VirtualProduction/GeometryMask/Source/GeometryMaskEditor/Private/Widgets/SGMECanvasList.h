// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FGMECanvasListViewModel;
class IGMETreeNodeViewModel;
class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

class SGMECanvasList
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGMECanvasList)
	{ }
	SLATE_END_ARGS()

	virtual ~SGMECanvasList() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<FGMECanvasListViewModel> InViewModel);

private:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IGMETreeNodeViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	void Refresh();
	
private:
	TSharedPtr<FGMECanvasListViewModel> ViewModel;
	
	TSharedPtr<SListView<TSharedPtr<IGMETreeNodeViewModel>>> CanvasListWidget;

	TArray<TSharedPtr<IGMETreeNodeViewModel>> CanvasItems;	
};
