// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FGMECanvasItemViewModel;
class IGMETreeNodeViewModel;
class ITableRow;
class STableViewBase;

template <typename InViewModelType>
class SGMEImageItem;

template <typename InViewModelType>
class SGMEImageItem : public STableRow<TSharedPtr<InViewModelType>>
{
	using FTableRowArgs = typename STableRow<TSharedPtr<InViewModelType>>::FArguments;

public:
	SLATE_BEGIN_ARGS(SGMEImageItem)
		{ }
		SLATE_ATTRIBUTE(FText, Label)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		constexpr float Padding = 2.0f;

		TAttribute<FOptionalSize> AspectRatioBinding;
		AspectRatioBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateSP(this, &SGMEImageItem::GetAspectRatio));

		TAttribute<FOptionalSize> HeightBinding;
		HeightBinding.Bind(TAttribute<FOptionalSize>::FGetter::CreateSP(this, &SGMEImageItem::GetHeight));

		FTableRowArgs Args = FTableRowArgs()
			.Padding(Padding)
			.ShowSelection(false)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				[
					SAssignNew(ImageContainerWidget, SBox)
					.HeightOverride(HeightBinding)
					.MinAspectRatio(AspectRatioBinding)
					.MaxAspectRatio(AspectRatioBinding)
					[
						ImageWidget.ToSharedRef()
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(1, 1, 1, Padding)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Font(FCoreStyle::GetDefaultFontStyle("Mono", 9))
						.Text(InArgs._Label)
					]
				]
			];

		STableRow<TSharedPtr<InViewModelType>>::Construct(Args, InOwnerTableView);
	}

protected:
	virtual FOptionalSize GetAspectRatio() = 0;

	/** Get desired height based on the actual width. */
	FOptionalSize GetHeight()
	{
		if (ImageContainerWidget.IsValid())
		{
			const FGeometry& WidgetGeometry = ImageContainerWidget->GetCachedGeometry();
			const float AspectRatio = 1.0f / GetAspectRatio().Get();
			const float Height = static_cast<int32>(WidgetGeometry.GetLocalSize().X) * AspectRatio;
			return Height;
		}

		return {};
	}

protected:
	TSharedPtr<SPanel> ImageContainerWidget;
	TSharedPtr<SWidget> ImageWidget;
};
