// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SGeometryMaskCanvasPreview.h"

#include "EditorSupportDelegates.h"
#include "Engine/CanvasRenderTarget2D.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "GeometryMaskCanvasResource.h"
#include "GeometryMaskEditorLog.h"
#include "GeometryMaskSubsystem.h"
#include "GeometryMaskTypes.h"
#include "GeometryMaskWorldSubsystem.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "SlateMaterialBrush.h"
#include "SlateOptMacros.h"
#include "UObject/Package.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"

namespace UE::GeometryMaskEditor::Private
{
	constexpr const TCHAR* PreviewMaterialPath = TEXT("/GeometryMask/GeometryMask/M_GeometryMaskPreview");
	constexpr const TCHAR* DefaultTexturePath = TEXT("/GeometryMask/GeometryMask/GeometryMaskDefaultRenderTarget");

	static FLazyName MaskTexturesParameterName = TEXT("Mask_Textures");
	static FLazyName MaskIndicesParameterName = TEXT("Mask_TextureIndexVector");
	static FLazyName InvertParameterName = TEXT("Mask_Invert");
	static FLazyName PaddingParameterName = TEXT("Mask_Padding");
	static FLazyName OpacityParameterName = TEXT("OpacityMultiplier");
	static FLazyName FeatherParameterName = TEXT("Feather");
}

SGeometryMaskCanvasPreview::SGeometryMaskCanvasPreview()
	: PreviewMaterialPath(UE::GeometryMaskEditor::Private::PreviewMaterialPath)
	, DefaultTexturePath(UE::GeometryMaskEditor::Private::DefaultTexturePath)
{
}

SGeometryMaskCanvasPreview::~SGeometryMaskCanvasPreview()
{
	FEditorSupportDelegates::PrepareToCleanseEditorObject.RemoveAll(this);

	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupDelegateHandle);
	OnWorldCleanupDelegateHandle.Reset();
}

void SGeometryMaskCanvasPreview::Construct(const FArguments& InArgs)
{
	OnWorldCleanupDelegateHandle = FWorldDelegates::OnWorldCleanup.AddSP(this, &SGeometryMaskCanvasPreview::OnWorldCleanup);

	CanvasId = InArgs._CanvasId;
	bShowPaddingFrame = InArgs._PaddingFrameVisibility;
	Invert = InArgs._Invert;
	HasSolidBackground = InArgs._SolidBackground;
	OpacityMultiplier = InArgs._Opacity;
	AspectRatio = 1920.0f / 1080.0f;

	// Setup MID and Brush
	{
		if (UMaterialInterface* PreviewMaterial = Cast<UMaterialInterface>(PreviewMaterialPath.TryLoad()))
		{
			PreviewMID = UMaterialInstanceDynamic::Create(PreviewMaterial, GetTransientPackage());
			PreviewMID->SetScalarParameterValue(UE::GeometryMaskEditor::Private::OpacityParameterName, OpacityMultiplier.Get(1.0f));
			PreviewBrush = MakeShared<FSlateMaterialBrush>(*PreviewMID, FVector2D(1920.f, 1080.f));
		}
		else
		{
			UE_LOGF(LogGeometryMaskEditor, Warning, "PreviewMaterial could not be loaded from path: '%ls'", *PreviewMaterialPath.ToString());
		}
	}

	// Load DefaultTexture
	{
		DefaultTexture = Cast<UTextureRenderTarget2DArray>(DefaultTexturePath.TryLoad());
		if (!DefaultTexture)
		{
			UE_LOGF(LogGeometryMaskEditor, Warning, "DefaultTexture could not be loaded from path: '%ls'", *DefaultTexturePath.ToString());
		}
	}

	UpdateBrush(nullptr);
    TryResolveCanvas();

	ChildSlot
	[
		SNew(SScaleBox)
		.Stretch(EStretch::ScaleToFill)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Visibility_Lambda([this]()
				{
					return GetSolidBackground()
						? EVisibility::Visible
						: EVisibility::Hidden;
				})
				.Image(FAppStyle::GetBrush("WhiteTexture"))
			]

			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(PreviewBrush.IsValid() ? PreviewBrush.Get() : FAppStyle::GetBrush("WhiteTexture"))
			]
		]
	];
}

void SGeometryMaskCanvasPreview::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);
	UpdateBrush(CanvasWeak.Get());
}

const FGeometryMaskCanvasId& SGeometryMaskCanvasPreview::GetCanvasId() const
{
	return CanvasId.Get(FGeometryMaskCanvasId::None);
}

void SGeometryMaskCanvasPreview::SetCanvasId(const FGeometryMaskCanvasId& InCanvasId)
{
	if (GetCanvasId() != InCanvasId)
	{
		CanvasId.Set(InCanvasId);
		TryResolveCanvas();
	}
}

const FName SGeometryMaskCanvasPreview::GetCanvasName() const
{
	return GetCanvasId().Name;
}

const EGeometryMaskColorChannel SGeometryMaskCanvasPreview::GetColorChannel(const bool bOnlyValid) const
{
	return EGeometryMaskColorChannel::Red;
}

void SGeometryMaskCanvasPreview::SetColorChannel(const EGeometryMaskColorChannel InColorChannel)
{
}

const bool SGeometryMaskCanvasPreview::IsPaddingFrameVisible() const
{
	return bShowPaddingFrame.Get(true);
}

void SGeometryMaskCanvasPreview::SetPaddingFrameVisiblity(const bool bInInverted)
{
	bShowPaddingFrame.Set(bInInverted);
}

const bool SGeometryMaskCanvasPreview::IsInverted() const
{
	return Invert.Get(false);
}

void SGeometryMaskCanvasPreview::SetInvert(const bool bInInverted)
{
	Invert.Set(bInInverted);
}

const bool SGeometryMaskCanvasPreview::GetSolidBackground() const
{
	return HasSolidBackground.Get();
}

void SGeometryMaskCanvasPreview::SetSolidBackground(const bool bInHasSolidBackground)
{
	HasSolidBackground.Set(bInHasSolidBackground);
}

const float SGeometryMaskCanvasPreview::GetOpacity() const
{
	return OpacityMultiplier.Get(1.0f);
}

void SGeometryMaskCanvasPreview::SetOpacity(const float InValue)
{
	OpacityMultiplier.Set(InValue);

	if (PreviewMID)
	{
		PreviewMID->SetScalarParameterValue(UE::GeometryMaskEditor::Private::OpacityParameterName, InValue);
	}
}

UGeometryMaskCanvas* SGeometryMaskCanvasPreview::GetCanvas() const
{
	return CanvasWeak.Get();
}

FOptionalSize SGeometryMaskCanvasPreview::GetAspectRatio()
{
	float Width = 1920.0f;
	float Height = 1080.0f;

	if (PreviewBrush.IsValid())
	{
		Width = PreviewBrush->ImageSize.X;
		Height = PreviewBrush->ImageSize.Y;		
	}
	
	if (AspectRatio.IsSet())
	{
		return AspectRatio.Get();
	}

	AspectRatio.Set(Width / Height);
	return AspectRatio.Get();
}

void SGeometryMaskCanvasPreview::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(PreviewMID);
	InCollector.AddReferencedObject(DefaultTexture);
}

FString SGeometryMaskCanvasPreview::GetReferencerName() const
{
	return TEXT("SGeometryMaskCanvasPreview");
}

bool SGeometryMaskCanvasPreview::TryResolveCanvas()
{
	if (!CanvasId.IsSet())
	{
		return false;
	}

	if (ULevel* CanvasLevel = CanvasId.Get().Level.ResolveObjectPtr())
	{
		if (CanvasLevel->OwningWorld)
		{
			if (UGeometryMaskWorldSubsystem* Subsystem = CanvasLevel->OwningWorld->GetSubsystem<UGeometryMaskWorldSubsystem>())
			{
				UGeometryMaskCanvas* Canvas = Subsystem->GetNamedCanvas(CanvasLevel, GetCanvasName());
				CanvasWeak = Canvas;
				UpdateBrush(Canvas);
			}
		}
	}

	return CanvasWeak != nullptr;
}

void SGeometryMaskCanvasPreview::UpdateBrush(const UGeometryMaskCanvas* InCanvas)
{
	UTextureRenderTarget2DArray* Texture = DefaultTexture;
	FLinearColor MaskIndices = FLinearColor(-1, -1, -1, -1);

	const bool bHasValidMID = PreviewMID != nullptr;

	// When a canvas is provided, update the parameters from it, vs just from this widget
	if (InCanvas)
	{
		if (UTextureRenderTarget2DArray* CanvasTexture = InCanvas->GetRenderTarget())
		{
			Texture = CanvasTexture;
			MaskIndices.R = static_cast<float>(InCanvas->GetRenderTargetSliceIndex());
		}

		FVector4f Padding(ForceInitToZero);
		if (const UGeometryMaskCanvasResource* CanvasResource = InCanvas->GetResource())
		{
			FIntPoint ViewportPadding(CanvasResource->GetSharedViewportPadding());
			FVector2f ViewportPaddingF = FVector2f(ViewportPadding) / CanvasResource->GetMaxViewportSize();
			Padding.X = ViewportPaddingF.X;
			Padding.Y = ViewportPaddingF.Y;
		}

		if (bHasValidMID)
		{
			const int32 MaxFeatherRadius = FMath::Max(InCanvas->GetOuterFeatherRadius(), InCanvas->GetInnerFeatherRadius());
			const FVector4f Feather(InCanvas->IsFeatherApplied() ? 1 : 0, InCanvas->GetOuterFeatherRadius(), InCanvas->GetInnerFeatherRadius(), MaxFeatherRadius);
			PreviewMID->SetVectorParameterValue(UE::GeometryMaskEditor::Private::FeatherParameterName, Feather);
			PreviewMID->SetVectorParameterValue(UE::GeometryMaskEditor::Private::PaddingParameterName, Padding);
		}
	}

	if (bHasValidMID)
	{
		PreviewMID->SetTextureParameterValue(UE::GeometryMaskEditor::Private::MaskTexturesParameterName, Texture);
		PreviewMID->SetVectorParameterValue(UE::GeometryMaskEditor::Private::MaskIndicesParameterName, MaskIndices);
		PreviewMID->SetScalarParameterValue(UE::GeometryMaskEditor::Private::InvertParameterName, IsInverted() ? 1.0f : 0.0f);
	}

	if (PreviewBrush.IsValid())
	{
		FVector2D ImageSize(1920.f, 1080.f);
		if (Texture)
		{
			ImageSize.X = Texture->GetSurfaceWidth();
			ImageSize.Y = Texture->GetSurfaceHeight();
		}

		PreviewBrush->ImageSize = ImageSize;
		AspectRatio.Set(ImageSize.X / ImageSize.Y);
	}
}

void SGeometryMaskCanvasPreview::OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources)
{
	const ULevel* const CanvasLevel = CanvasId.Get().Level.ResolveObjectPtr();
	if (!CanvasLevel || CanvasLevel->GetWorld() == InWorld || CanvasLevel->GetOuter() == InWorld)
	{
		UpdateBrush(/*Canvas*/nullptr);
	}
}
