// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryMaskTypes.h"
#include "Types/SlateStructs.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

class UGeometryMaskCanvas;
class UMaterialInstanceDynamic;
class UTextureRenderTarget2DArray;

/** Displays a named GeometryMaskCanvas. */
class GEOMETRYMASKEDITOR_API SGeometryMaskCanvasPreview
	: public SCompoundWidget
	, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SGeometryMaskCanvasPreview) {}
		SLATE_ATTRIBUTE(FGeometryMaskCanvasId, CanvasId)
		SLATE_ATTRIBUTE_DEPRECATED(EGeometryMaskColorChannel, Channel, 5.8, "Color channel management has been deprecated as it's no longer needed")
		SLATE_ATTRIBUTE(bool, PaddingFrameVisibility)
		SLATE_ATTRIBUTE(bool, Invert)
		SLATE_ATTRIBUTE(bool, SolidBackground)
		SLATE_ATTRIBUTE(float, Opacity)
	SLATE_END_ARGS()

	SGeometryMaskCanvasPreview();
	virtual ~SGeometryMaskCanvasPreview() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Get the id of the currently referenced canvas. */
	const FGeometryMaskCanvasId& GetCanvasId() const;

	/** Sets the id of the currently referenced canvas, and resolves the canvas itself. */
	void SetCanvasId(const FGeometryMaskCanvasId& InCanvasId);

	/** Get the name of the currently referenced canvas. */
	const FName GetCanvasName() const;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	const EGeometryMaskColorChannel GetColorChannel(const bool bOnlyValid = true) const;

	UE_DEPRECATED(5.8, "Color channel management has been deprecated as it's no longer needed")
	void SetColorChannel(const EGeometryMaskColorChannel InColorChannel);

	const bool IsPaddingFrameVisible() const;

	void SetPaddingFrameVisiblity(const bool bInIsVisible);

	/** Get whether to invert the display of the canvas.  */
	const bool IsInverted() const;

	/** Set whether to invert the display of the canvas.  */
	void SetInvert(const bool bInInverted);

	/** Get whether a solid background is used or not (vs. compositing with alpha on widgets below). */
	const bool GetSolidBackground() const;

	/** Set whether a solid background is used or not (vs. compositing with alpha on widgets below).  */
	void SetSolidBackground(const bool bInHasSolidBackground);

	/** Get the overall opacity multiplier value. */
	const float GetOpacity() const;

	/** Set the overall opacity multiplier value. */
	void SetOpacity(const float InValue);

	/** Get the currently referenced canvas. */
	UGeometryMaskCanvas* GetCanvas() const;

	/** Get the aspect ratio of the referenced canvas. */
	FOptionalSize GetAspectRatio();

	// ~Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	virtual FString GetReferencerName() const override;
	// ~End FGCObject Interface

private:
	bool TryResolveCanvas();

	void UpdateBrush(const UGeometryMaskCanvas* InCanvas);

	/**
	 * Called when the world is being cleaned up
	 * Used to remove the mask texture reference from the preview MIDs if the textures belong to the world being cleaned up
	 */
	void OnWorldCleanup(UWorld* InWorld, bool bInSessionEnded, bool bInCleanupResources);

private:
	TSharedPtr<FSlateBrush> PreviewBrush;
	FSoftObjectPath PreviewMaterialPath;
	TObjectPtr<UMaterialInstanceDynamic> PreviewMID;

	FSoftObjectPath DefaultTexturePath;
	TObjectPtr<UTextureRenderTarget2DArray> DefaultTexture;

	TWeakObjectPtr<UGeometryMaskCanvas> CanvasWeak;

	/** Id of the currently referenced canvas. */
	TAttribute<FGeometryMaskCanvasId> CanvasId;

	TAttribute<bool> bShowPaddingFrame;

	/** Whether to invert the display of the canvas.  */
	TAttribute<bool> Invert;

	/*** Whether a solid background is used or not (vs. compositing with alpha on widgets below). */
	TAttribute<bool> HasSolidBackground;

	/** Multiplies the overall opacity. */
	TAttribute<float> OpacityMultiplier;

	/** Stores the aspect ratio of the referenced canvas. */
	TAttribute<FOptionalSize> AspectRatio;

	/** Handle to the delegate called when world gets cleaned up */
	FDelegateHandle OnWorldCleanupDelegateHandle;
};
